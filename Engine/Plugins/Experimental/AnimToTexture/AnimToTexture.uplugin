{
	"FileVersion": 3,
	"Version": 2,
	"VersionName": "2.1",
	"FriendlyName": "AnimToTexture",
	"Description": "Converts SkeletalMesh Animations into Textures",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimToTexture",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "AnimToTextureEditor",
			"Type": "Editor",
			"LoadingPhase": "PreLoadingScreen"
		}
	]
}