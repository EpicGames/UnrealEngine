[CoreRedirects]
+StructRedirects=(OldName="/Script/AnimToTexture.AnimSequenceInfo",NewName="/Script/AnimToTexture.AnimToTextureAnimSequenceInfo")
+StructRedirects=(OldName="/Script/AnimToTexture.AnimInfo",NewName="/Script/AnimToTexture.AnimToTextureAnimInfo")
+PropertyRedirects==(OldName="AnimToTextureDataAsset.MasterSkeletalMesh",NewName="LeaderSkeletalMesh")
+FunctionRedirects=(OldName="AnimToTextureDataAsset.BP_GetMasterSkeletalMesh", NewName="BP_GetLeaderSkeletalMesh")
