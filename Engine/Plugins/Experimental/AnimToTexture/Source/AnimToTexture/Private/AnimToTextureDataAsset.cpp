// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimToTextureDataAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimToTextureDataAsset)

int32 UAnimToTextureDataAsset::GetIndexFromAnimSequence(const UAnimSequence* Sequence)
{
	const int32 NumSequences = AnimSequences.Num();
	
	// We can store a sequence to index map for a faster search
	for (int32 Index = 0; Index < NumSequences; ++Index)
	{
		const FAnimToTextureAnimSequenceInfo& SequenceInfo = AnimSequences[Index];
		if (SequenceInfo.bEnabled && SequenceInfo.AnimSequence == Sequence)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

void UAnimToTextureDataAsset::ResetInfo()
{
	// Common Info.
	NumFrames = 0;
	Animations.Reset();

	// Vertex Info
	VertexRowsPerFrame = 1;
	VertexMinBBox = FVector3f::ZeroVector;
	VertexSizeBBox = FVector3f::ZeroVector;

	// Bone Info
	NumBones = 0;
	BoneRowsPerFrame = 1;
	BoneWeightRowsPerFrame = 1;
	BoneMinBBox = FVector3f::ZeroVector;
	BoneSizeBBox = FVector3f::ZeroVector;
};
