// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimToTextureInstancePlaybackHelpers.h"
#include "AnimToTextureDataAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimToTextureInstancePlaybackHelpers)

bool UAnimToTextureInstancePlaybackLibrary::SetupInstancedMeshComponent(UInstancedStaticMeshComponent* InstancedMeshComponent, int32 NumInstances, bool bAutoPlay)
{	
	if (!InstancedMeshComponent)
	{
		return false;
	}

	// Clear data.
	InstancedMeshComponent->ClearInstances();

	if (!NumInstances)
	{
		return false;
	}

	// Allocate Transforms
	TArray<FTransform> InstanceTransforms;
	InstanceTransforms.AddDefaulted(NumInstances);

	// Set Custom Data Length
	const SIZE_T DataSize = bAutoPlay ? sizeof(FAnimToTextureAutoPlayData) : sizeof(FAnimToTextureFrameData);
	InstancedMeshComponent->NumCustomDataFloats = DataSize / sizeof(float);

	// Initizalize Instances
	InstancedMeshComponent->AddInstances(InstanceTransforms, /*bShouldReturnIndices*/ false, /*bWorldSpace*/ false);

	return true;
}

bool UAnimToTextureInstancePlaybackLibrary::BatchUpdateInstancesAutoPlayData(UInstancedStaticMeshComponent* InstancedMeshComponent, 
	const TArray<FAnimToTextureAutoPlayData>& AutoPlayData, const TArray<FTransform>& Transforms, bool bMarkRenderStateDirty)
{
	return BatchUpdateInstancesData< FAnimToTextureAutoPlayData>(InstancedMeshComponent, AutoPlayData, Transforms, bMarkRenderStateDirty);
}

bool UAnimToTextureInstancePlaybackLibrary::BatchUpdateInstancesFrameData(UInstancedStaticMeshComponent* InstancedMeshComponent,
	const TArray<FAnimToTextureFrameData>& FrameData, const TArray<FTransform>& Transforms, bool bMarkRenderStateDirty)
{
	return BatchUpdateInstancesData<FAnimToTextureFrameData>(InstancedMeshComponent, FrameData, Transforms, bMarkRenderStateDirty);
}

bool UAnimToTextureInstancePlaybackLibrary::UpdateInstanceAutoPlayData(UInstancedStaticMeshComponent* InstancedMeshComponent,
	int32 InstanceIndex, const FAnimToTextureAutoPlayData& AutoPlayData, bool bMarkRenderStateDirty)
{
	return UpdateInstanceCustomData<FAnimToTextureAutoPlayData>(InstancedMeshComponent, InstanceIndex, AutoPlayData, bMarkRenderStateDirty);
}

bool UAnimToTextureInstancePlaybackLibrary::UpdateInstanceFrameData(UInstancedStaticMeshComponent* InstancedMeshComponent,
	int32 InstanceIndex, const FAnimToTextureFrameData& FrameData, bool bMarkRenderStateDirty)
{
	return UpdateInstanceCustomData<FAnimToTextureFrameData>(InstancedMeshComponent, InstanceIndex, FrameData, bMarkRenderStateDirty);
}

bool UAnimToTextureInstancePlaybackLibrary::GetAutoPlayDataFromDataAsset(const UAnimToTextureDataAsset* DataAsset, int32 AnimationIndex,
	FAnimToTextureAutoPlayData& AutoPlayData, float TimeOffset, float PlayRate)
{
	if (DataAsset && DataAsset->Animations.IsValidIndex(AnimationIndex))
	{
		// Copy Frame Range
		const FAnimToTextureAnimInfo& AnimInfo = DataAsset->Animations[AnimationIndex];
		AutoPlayData.StartFrame = AnimInfo.StartFrame;
		AutoPlayData.EndFrame = AnimInfo.EndFrame;

		AutoPlayData.TimeOffset = TimeOffset;
		AutoPlayData.PlayRate = PlayRate;

		return true;
	}

	// Return default
	AutoPlayData = FAnimToTextureAutoPlayData();
	return false;
}

float UAnimToTextureInstancePlaybackLibrary::GetFrame(float Time, float StartFrame, float EndFrame, 
	float TimeOffset, float PlayRate, float SampleRate)
{	
	// Clamp inputs (just in case)
	Time = FMath::Max(Time, 0.f);
	TimeOffset = FMath::Max(TimeOffset, 0.f);
	PlayRate = FMath::Max(PlayRate, 0.f);

	const float Frame = (Time + TimeOffset) * (PlayRate * SampleRate);
	const float NumFrames = EndFrame - StartFrame + 1.f;
	return FMath::Fmod(Frame, NumFrames) + StartFrame;
}

bool UAnimToTextureInstancePlaybackLibrary::GetFrameDataFromDataAsset(const UAnimToTextureDataAsset* DataAsset, int32 AnimationIndex, float Time, FAnimToTextureFrameData& FrameData, float TimeOffset, float PlayRate)
{
	if (DataAsset && DataAsset->Animations.IsValidIndex(AnimationIndex))
	{
		// Copy Frame Range
		const FAnimToTextureAnimInfo& AnimInfo = DataAsset->Animations[AnimationIndex];
		
		FrameData.Frame = GetFrame(Time, AnimInfo.StartFrame, AnimInfo.EndFrame, 
			TimeOffset, PlayRate, DataAsset->SampleRate);

		// Previous Frame
		FrameData.PrevFrame = FMath::Clamp(FrameData.Frame - 1, AnimInfo.StartFrame, AnimInfo.EndFrame);

		return true;
	}

	// Return default
	FrameData = FAnimToTextureFrameData();
	return false;
}
