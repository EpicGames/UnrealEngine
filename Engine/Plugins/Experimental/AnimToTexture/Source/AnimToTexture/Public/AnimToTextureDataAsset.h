// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "AnimToTextureDataAsset.generated.h"

#define UE_API ANIMTOTEXTURE_API

class USkeletalMesh;
class UStaticMesh;
class UTexture2D;

namespace AnimToTextureParamNames
{
	static const FName Frame = TEXT("Frame");
	static const FName AutoPlay = TEXT("AutoPlay");
	static const FName StartFrame = TEXT("StartFrame");
	static const FName EndFrame = TEXT("EndFrame");
	static const FName SampleRate = TEXT("SampleRate");
	static const FName NumFrames = TEXT("NumFrames");
	static const FName MinBBox = TEXT("MinBBox");
	static const FName SizeBBox = TEXT("SizeBBox");
	static const FName NumBones = TEXT("NumBones");
	static const FName RowsPerFrame = TEXT("RowsPerFrame");
	static const FName BoneWeightRowsPerFrame = TEXT("BoneWeightsRowsPerFrame");
	static const FName VertexPositionTexture = TEXT("PositionTexture");
	static const FName VertexNormalTexture = TEXT("NormalTexture");
	static const FName BonePositionTexture = TEXT("BonePositionTexture");
	static const FName BoneRotationTexture = TEXT("BoneRotationTexture");
	static const FName BoneWeightsTexture = TEXT("BoneWeightsTexture");
	static const FName UseUV0 = TEXT("UseUV0");
	static const FName UseUV1 = TEXT("UseUV1");
	static const FName UseUV2 = TEXT("UseUV2");
	static const FName UseUV3 = TEXT("UseUV3");
	static const FName UseTwoInfluences = TEXT("UseTwoInfluences");
	static const FName UseFourInfluences = TEXT("UseFourInfluences");
}

UENUM(Blueprintable)
enum class EAnimToTextureMode : uint8
{
	/* Position and Normal Per-Vertex */
	Vertex,
	/* Linear Blending Skinnin */
	Bone,
};

UENUM(Blueprintable)
enum class EAnimToTexturePrecision : uint8
{
	/* 8 bits */
	EightBits,
	/* 16 bits */
	SixteenBits,
};

UENUM(Blueprintable)
enum class EAnimToTextureNumBoneInfluences : uint8
{
	/* Single bone influence */
	One,
	/* Blend between two influences */
	Two,
	/* Blend between four influences */
	Four,
};

USTRUCT(Blueprintable)
struct FAnimToTextureAnimSequenceInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default, BlueprintReadWrite)
	bool bEnabled = true;

	UPROPERTY(EditAnywhere, Category = Default, BlueprintReadWrite, meta = (EditCondition = "bEnabled", EditConditionHides))
	TObjectPtr<UAnimSequence> AnimSequence = nullptr;
	
	/* Use Custom FrameRange */
	UPROPERTY(EditAnywhere, Category = Default, BlueprintReadWrite, meta = (EditCondition = "bEnabled", EditConditionHides))
	bool bUseCustomRange = false;

	/* Animation Start Frame */
	UPROPERTY(EditAnywhere, Category = Default, BlueprintReadWrite, meta = (EditCondition = "bEnabled && bUseCustomRange", EditConditionHides))
	int32 StartFrame = 0;

	/* Animation End Frame (Inclusive) */
	UPROPERTY(EditAnywhere, Category = Default, BlueprintReadWrite, meta = (EditCondition = "bEnabled && bUseCustomRange", EditConditionHides))
	int32 EndFrame = 0;

};

USTRUCT(Blueprintable)
struct FAnimToTextureAnimInfo
{
	GENERATED_BODY()
	
	UPROPERTY(VisibleAnywhere, Category = Default, BlueprintReadOnly)
	int32 StartFrame = 0;

	UPROPERTY(VisibleAnywhere, Category = Default, BlueprintReadOnly)
	int32 EndFrame = 0;

};

UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UAnimToTextureDataAsset : public UPrimaryDataAsset
{
public:
	GENERATED_BODY()

	/**
	* SkeletalMesh to bake animations from.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SkeletalMesh", meta = (AssetBundles = "Client"))
	TSoftObjectPtr<USkeletalMesh> SkeletalMesh;

	/**
	* SkeletalMesh LOD to bake.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SkeletalMesh", Meta = (DisplayName = "LODIndex"))
	int32 SkeletalLODIndex = 0;

	/**
	* StaticMesh to bake to.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StaticMesh", meta = (AssetBundles = "Client"))
	TSoftObjectPtr<UStaticMesh> StaticMesh;

	/**
	* StaticMesh LOD to bake to.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StaticMesh", Meta = (DisplayName = "LODIndex"))
	int32 StaticLODIndex = 0;

	/**
	* StaticMesh UVChannel Index for storing vertex information.
	* Make sure this index does not conflict with the Lightmap UV Index.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StaticMesh")
	int32 UVChannel = 1;

	/**
	* Number of Driver Triangles
	* Each StaticMesh Vertex will be influenced by N SkeletalMesh (Driver) Triangles.
	* Increasing the Number of Driver Triangles will increase the Mapping computation.
	* Using a single Driver Triangle will do a Rigid Binding to Closest Triangle.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StaticMesh|Mapping")
	int32 NumDriverTriangles = 10;

	/**
	* Inverse Distance Weighting
	* This exponent value will be used for computing weights for the DriverTriangles.
	* Larger number will create a more contrasted weighting, but it might 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StaticMesh|Mapping")
	float Sigma = 1.f;

	// ------------------------------------------------------
	// Texture

	/**
	* Max resolution of the texture.
	* A smaller size will be used if the data fits.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture")
	int32 MaxHeight = 4096;

	/**
	* Max resolution of the texture.
	* A smaller size will be used if the data fits.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture")
	int32 MaxWidth = 4096;

	/**
	* Enforce Power Of Two on texture resolutions.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture")
	bool bEnforcePowerOfTwo = false;

	/**
	* Texture Precision
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture")
	EAnimToTexturePrecision Precision = EAnimToTexturePrecision::EightBits;

	/**
	* Storage Mode.
	* Vertex: will store per-vertex position and normal.
	* Bone: Will store per-bone position and rotation and per-vertex bone weight. 
	        This is the preferred method if meshes share skeleton.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture")
	EAnimToTextureMode Mode = EAnimToTextureMode::Bone;

	/**
	* Texture for storing vertex positions
	* This is only used on Vertex Mode
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", meta = (EditCondition = "Mode == EAnimToTextureMode::Vertex", EditConditionHides))
	TSoftObjectPtr<UTexture2D> VertexPositionTexture;

	/**
	* Texture for storing vertex normals
	* This is only used on Vertex Mode
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", meta = (EditCondition = "Mode == EAnimToTextureMode::Vertex", EditConditionHides))
	TSoftObjectPtr<UTexture2D> VertexNormalTexture;

	/**
	* Texture for storing bone positions
	* This is only used on Bone Mode
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", meta = (EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	TSoftObjectPtr<UTexture2D> BonePositionTexture;

	/**
	* Texture for storing bone rotations
	* This is only used on Bone Mode
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", meta = (EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	TSoftObjectPtr<UTexture2D> BoneRotationTexture;

	/**
	* Texture for storing vertex/bone weighting
	* This is only used on Bone Mode
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture", meta = (EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	TSoftObjectPtr<UTexture2D> BoneWeightTexture;

	// ------------------------------------------------------
	// Animation
	
	/**
	* Adds transformation to baked textures. 
	* This can be used for adding an offset to the animation.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation")
	FTransform RootTransform;

	/** 
	* Bone used for Rigid Binding. The bone needs to be part of the RawBones. 
	* Sockets and VirtualBones are not supported.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation", meta = (EditCondition = "Mode == EAnimToTextureMode::Bone"))
	FName AttachToSocket;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation")
	float SampleRate = 30.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation")
	TArray<FAnimToTextureAnimSequenceInfo> AnimSequences;

	// ------------------------------------------------------
	// Material Layer Static Switches

	/**
	* AutoPlay will use Engine Time for driving the animation.
	* This will be used by UpdateMaterialInstanceFromDataAsset and AssetActions for setting MaterialInstance static switches
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material")
	bool bAutoPlay = true;

	/**
	* AnimationIndex Index of the animation to play.
	* This will internally set Start and End Frame when using AutoPlay.
	* This will be used by UpdateMaterialInstanceFromDataAsset and AssetActions for setting MaterialInstance static switches
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material", meta = (EditCondition = "bAutoPlay"))
	int32 AnimationIndex = 0;

	/**
	* Frame to play
	* When not using AutoPlay, user is responsible of setting the frame.
	* This will be used by UpdateMaterialInstanceFromDataAsset and AssetActions for setting MaterialInstance static switches
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material", meta = (EditCondition = "!bAutoPlay"))
	int32 Frame = 0;

	/**
	* Number of Bone Influences for deformation. More influences will produce smoother results at the cost of performance.
	* This will be used by UpdateMaterialInstanceFromDataAsset and AssetActions for setting MaterialInstance static switches
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material", meta = (EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	EAnimToTextureNumBoneInfluences NumBoneInfluences = EAnimToTextureNumBoneInfluences::Four;

	// ------------------------------------------------------
	// Info

	/* Total Number of Frames in all animations */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info")
	int32 NumFrames = 0;

	/* Total Number of Bones */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info", Meta = (EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	int32 NumBones = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info", Meta = (EditCondition = "Mode == EAnimToTextureMode::Vertex", EditConditionHides))
	int32 VertexRowsPerFrame = 1;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info", Meta = (DisplayName = "MinBBox", EditCondition = "Mode == EAnimToTextureMode::Vertex", EditConditionHides))
	FVector3f VertexMinBBox;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info", Meta = (DisplayName = "SizeBBox", EditCondition = "Mode == EAnimToTextureMode::Vertex", EditConditionHides))
	FVector3f VertexSizeBBox;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info", Meta = (EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	int32 BoneWeightRowsPerFrame = 1;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info", Meta = (EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	int32 BoneRowsPerFrame = 1;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info", Meta = (DisplayName = "MinBBox", EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	FVector3f BoneMinBBox;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info", Meta = (DisplayName = "SizeBBox", EditCondition = "Mode == EAnimToTextureMode::Bone", EditConditionHides))
	FVector3f BoneSizeBBox;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Info")
	TArray<FAnimToTextureAnimInfo> Animations;

	/* Finds AnimSequence Index in the Animations Array. 
	*  Only Enabled elements are returned.
	*  Returns -1 if not found.
	*/
	UFUNCTION(BlueprintCallable, Category = Default)
	UE_API int32 GetIndexFromAnimSequence(const UAnimSequence* Sequence);

	UFUNCTION()
	UE_API void ResetInfo();

	// If we weren't in a plugin, we could unify this in a base class
	template<typename AssetType>
	static AssetType* GetAsset(const TSoftObjectPtr<AssetType>& AssetPointer)
	{
		AssetType* ReturnVal = nullptr;
		if (AssetPointer.ToSoftObjectPath().IsValid())
		{
			ReturnVal = AssetPointer.Get();
			if (!ReturnVal)
			{
				AssetType* LoadedAsset = Cast<AssetType>(AssetPointer.ToSoftObjectPath().TryLoad());
				if (ensureMsgf(LoadedAsset, TEXT("Failed to load asset pointer %s"), *AssetPointer.ToString()))
				{
					ReturnVal = LoadedAsset;
				}
			}
		}
		return ReturnVal;
	}

#define AnimToTextureDataAsset_ASSET_ACCESSOR(ClassName, PropertyName) \
	inline ClassName* Get##PropertyName() const { return GetAsset(PropertyName); }

	AnimToTextureDataAsset_ASSET_ACCESSOR(UStaticMesh, StaticMesh);
	AnimToTextureDataAsset_ASSET_ACCESSOR(USkeletalMesh, SkeletalMesh);
	AnimToTextureDataAsset_ASSET_ACCESSOR(UTexture2D, VertexPositionTexture);
	AnimToTextureDataAsset_ASSET_ACCESSOR(UTexture2D, VertexNormalTexture);
	AnimToTextureDataAsset_ASSET_ACCESSOR(UTexture2D, BonePositionTexture);
	AnimToTextureDataAsset_ASSET_ACCESSOR(UTexture2D, BoneRotationTexture);
	AnimToTextureDataAsset_ASSET_ACCESSOR(UTexture2D, BoneWeightTexture);

	UFUNCTION(BlueprintPure, Category = Default, meta = (DisplayName = "Get Static Mesh"))
	UStaticMesh* BP_GetStaticMesh() { return GetStaticMesh(); }

	UFUNCTION(BlueprintPure, Category = Default, meta = (DisplayName = "Get Skeletal Mesh"))
	USkeletalMesh* BP_GetSkeletalMesh() { return GetSkeletalMesh(); }

	UFUNCTION(BlueprintPure, Category = Default, meta = (DisplayName = "Get Bone Position Texture"))
	UTexture2D* BP_GetBonePositionTexture() { return GetBonePositionTexture(); }

	UFUNCTION(BlueprintPure, Category = Default, meta = (DisplayName = "Get Bone Rotation Texture"))
	UTexture2D* BP_GetBoneRotationTexture() { return GetBoneRotationTexture(); }

	UFUNCTION(BlueprintPure, Category = Default, meta = (DisplayName = "Get Bone Weight Texture"))
	UTexture2D* BP_GetBoneWeightTexture() { return GetBoneWeightTexture(); }
};

#undef UE_API
