// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "AnimToTextureInstancePlaybackHelpers.generated.h"

#define UE_API ANIMTOTEXTURE_API

class UAnimToTextureDataAsset;

USTRUCT(BlueprintType)
struct FAnimToTextureFrameData
{
	GENERATED_USTRUCT_BODY()

	/**
	* Frame to be played
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimToTexture|Playback")
	float Frame = 0.0f;

	/**
	* Previous Frame (this is needeed for motion blur)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimToTexture|Playback")
	float PrevFrame = 0.0f;
};

USTRUCT(BlueprintType)
struct FAnimToTextureAutoPlayData
{
	GENERATED_USTRUCT_BODY()

	/**
	* Adds offset to time
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimToTexture|Playback")
	float TimeOffset = 0.0f;

	/**
	* Rate for increasing and decreasing speed.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimToTexture|Playback")
	float PlayRate = 1.0f;

	/**
	* Starting frame for animation. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimToTexture|Playback")
	float StartFrame = 0.0f;

	/**
	* Last frame of animation
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimToTexture|Playback")
	float EndFrame = 1.0f;
};


UCLASS(MinimalAPI)
class UAnimToTextureInstancePlaybackLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	
	/**
	* Adds Instances and allocates the necessary CustomData.
	* @param bAutoPlay: if true, it will allocate the required CustomData for working with AutoPlayData. If false, FrameData will be allocated instead./
	*/
	UFUNCTION(BlueprintCallable, Category = "AnimToTexture|Playback")
	static UE_API bool SetupInstancedMeshComponent(UInstancedStaticMeshComponent* InstancedMeshComponent, int32 NumInstances, bool bAutoPlay);

	/**
	* Updates all instances with the given Transforms and AutoPlayData
	* @param bMarkRenderStateDirty: if true, the change should be visible immediatelly. If you are updating many instances you should only set this to true for the last instance
	*/
	UFUNCTION(BlueprintCallable, Category = "AnimToTexture|Playback")
	static UE_API bool BatchUpdateInstancesAutoPlayData(UInstancedStaticMeshComponent* InstancedMeshComponent,
		const TArray<FAnimToTextureAutoPlayData>& AutoPlayData, const TArray<FTransform>& Transforms, bool bMarkRenderStateDirty=true);

	/**
	* Updates all instances with the given Transforms and FrameData
	* @param bMarkRenderStateDirty: if true, the change should be visible immediatelly. If you are updating many instances you should only set this to true for the last instance
	*/
	UFUNCTION(BlueprintCallable, Category = "AnimToTexture|Playback")
	static UE_API bool BatchUpdateInstancesFrameData(UInstancedStaticMeshComponent* InstancedMeshComponent,
		const TArray<FAnimToTextureFrameData>& FrameData, const TArray<FTransform>& Transforms, bool bMarkRenderStateDirty=true);

	/**
	* Updates a single instance with given AutoPlayData
	* @param bMarkRenderStateDirty: if true, the change should be visible immediatelly. If you are updating many instances you should only set this to true for the last instance
	*/
	UFUNCTION(BlueprintCallable, Category = "AnimToTexture|Playback")
	static UE_API bool UpdateInstanceAutoPlayData(UInstancedStaticMeshComponent* InstancedMeshComponent, int32 InstanceIndex, 
		const FAnimToTextureAutoPlayData& AutoPlayData, bool bMarkRenderStateDirty=true);

	/**
	* * Updates a single instance with given FrameData
	* @param bMarkRenderStateDirty: if true, the change should be visible immediatelly. If you are updating many instances you should only set this to true for the last instance
	*/
	UFUNCTION(BlueprintCallable, Category = "AnimToTexture|Playback")
	static UE_API bool UpdateInstanceFrameData(UInstancedStaticMeshComponent* InstancedMeshComponent, int32 InstanceIndex, 
		const FAnimToTextureFrameData& FrameData, bool bMarkRenderStateDirty=true);

	/**
	* Returns an AutoPlayData with the Start and End Frame for the given AnimationIndex.
	* If AnimationIndex is out of range, false will be returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "AnimToTexture|Playback")
	static UE_API bool GetAutoPlayDataFromDataAsset(const UAnimToTextureDataAsset* DataAsset, int32 AnimationIndex,
		FAnimToTextureAutoPlayData& AutoPlayData, float TimeOffset = 0.f, float PlayRate = 1.f);

	UFUNCTION(BlueprintCallable, Category = "AnimToTexture|Playback")
	static UE_API float GetFrame(float Time, float StartFrame, float EndFrame, 
		float TimeOffset = 0.f, float PlayRate = 1.f, float SampleRate = 30.f);

	UFUNCTION(BlueprintCallable, Category = "AnimToTexture|Playback")
	static UE_API bool GetFrameDataFromDataAsset(const UAnimToTextureDataAsset* DataAsset, int32 AnimationIndex, float Time,
		FAnimToTextureFrameData& FrameData, float TimeOffset = 0.f, float PlayRate = 1.f);
	
private:
	
	template <class DataType>
	static bool BatchUpdateInstancesData(UInstancedStaticMeshComponent* InstancedMeshComponent,
		const TArray<DataType>& Data, const TArray<FTransform>& Transforms, bool bMarkRenderStateDirty = true);
	
	template <class DataType>
	static bool UpdateInstanceCustomData(UInstancedStaticMeshComponent* InstancedMeshComponent, int32 InstanceIndex,
		const DataType& Data, bool bMarkRenderStateDirty = true);

	template <class DataType>
	static void GetData(const DataType& Data, TArray<float>& CustomData);

	template <class DataType>
	static void GetCustomData(const TArray<DataType>& Data, TArray<float>& CustomData);

};


template <class DataType>
bool UAnimToTextureInstancePlaybackLibrary::BatchUpdateInstancesData(UInstancedStaticMeshComponent* InstancedMeshComponent,
	const TArray<DataType>& Data, const TArray<FTransform>& Transforms, bool bMarkRenderStateDirty)
{
	if (!InstancedMeshComponent)
	{
		return false;
	}

	// Get Number of Instances
	const int32 NumInstances = InstancedMeshComponent->GetNumRenderInstances();
	const int32 NumCustomDataFloats = InstancedMeshComponent->NumCustomDataFloats;
	
	// Check number of instances
	if (NumInstances &&
		NumInstances == Data.Num() &&
		NumInstances == Transforms.Num() &&
		NumCustomDataFloats == Data.GetTypeSize() / sizeof(float))
	{
		// Set Transforms
		// TODO: we should be passing PrevTransforms too.
		InstancedMeshComponent->BatchUpdateInstancesTransforms(0, Transforms, false, bMarkRenderStateDirty, false);

		// Set Custom Data
		TArray<float> CustomData;
		GetCustomData(Data, CustomData);
		InstancedMeshComponent->SetCustomData(0, NumInstances-1, CustomData, bMarkRenderStateDirty);

		return true;
	}

	return false;
}

template <class DataType>
void UAnimToTextureInstancePlaybackLibrary::GetData(const DataType& Data, TArray<float>& CustomData)
{
	const SIZE_T DataSize = sizeof(Data);
	CustomData.SetNumUninitialized(DataSize / sizeof(float));

	FMemory::Memcpy(CustomData.GetData(), &Data, DataSize);
}

template <class DataType>
void UAnimToTextureInstancePlaybackLibrary::GetCustomData(const TArray<DataType>& Data, TArray<float>& CustomData)
{
	const int32 NumCustomDataFloats = Data.GetTypeSize() / sizeof(float);
	CustomData.SetNumUninitialized(Data.Num() * NumCustomDataFloats);
	
	FMemory::Memcpy(CustomData.GetData(), Data.GetData(), Data.Num() * Data.GetTypeSize());
}

template <class DataType>
bool UAnimToTextureInstancePlaybackLibrary::UpdateInstanceCustomData(UInstancedStaticMeshComponent* InstancedMeshComponent, int32 InstanceIndex,
	const DataType& Data, bool bMarkRenderStateDirty)
{
	if (InstancedMeshComponent && InstancedMeshComponent->IsValidInstance(InstanceIndex))
	{
		// Check if valid size
		const int32 NumCustomDataFloats = sizeof(Data) / sizeof(float);
		if (InstancedMeshComponent->NumCustomDataFloats == NumCustomDataFloats)
		{
			TArray<float> CustomData;
			GetData(Data, CustomData);
			return InstancedMeshComponent->SetCustomData(InstanceIndex, CustomData, bMarkRenderStateDirty);
		}
	}

	return false;
}

#undef UE_API
