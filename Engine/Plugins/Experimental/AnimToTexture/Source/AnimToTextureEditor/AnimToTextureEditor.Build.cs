// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimToTextureEditor : ModuleRules
{
	public AnimToTextureEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"Core",
				"MaterialEditor",
				"MessageLog"
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetTools",
				"ContentBrowser",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"AnimToTexture",
				"RawMesh",
				"MeshDescription",
				//"SkeletalMeshDescription",
				"StaticMeshDescription",
				"ToolMenus",
				"UnrealEd",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
