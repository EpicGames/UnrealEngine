// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimToTextureBPLibrary.h"
#include "AnimToTextureEditorModule.h"
#include "AnimToTextureUtils.h"
#include "AnimToTextureSkeletalMesh.h"

#include "LevelEditor.h"
#include "RawMesh.h"
#include "MeshUtilities.h"
#include "Modules/ModuleManager.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Math/Vector.h"
#include "Math/NumericLimits.h"
#include "MeshDescription.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialEditingLibrary.h"
#include "Misc/ScopedSlowTask.h"

#define LOCTEXT_NAMESPACE "AnimToTextureEditor"

using namespace AnimToTexture_Private;

UAnimToTextureBPLibrary::UAnimToTextureBPLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

bool UAnimToTextureBPLibrary::AnimationToTexture(UAnimToTextureDataAsset* DataAsset)
{
	if (!DataAsset)
	{
		return false;
	}

	// Runs some checks for the assets in DataAsset
	int32 SocketIndex = INDEX_NONE;
	TArray<FAnimToTextureAnimSequenceInfo> AnimSequences;
	if (!CheckDataAsset(DataAsset, SocketIndex, AnimSequences))
	{
		return false;
	}

	// Reset DataAsset Info Values
	DataAsset->ResetInfo();

	// ---------------------------------------------------------------------------		
	// Get Mapping between Static and Skeletal Meshes
	// Since they might not have same number of points.
	//
	FSourceMeshToDriverMesh Mapping;
	{
		FScopedSlowTask ProgressBar(1.f, LOCTEXT("ProcessingMapping", "Processing StaticMesh -> SkeletalMesh Mapping ..."), true /*Enabled*/);
		ProgressBar.MakeDialog(false /*bShowCancelButton*/, false /*bAllowInPIE*/);

		Mapping.Update(DataAsset->GetStaticMesh(), DataAsset->StaticLODIndex,
			DataAsset->GetSkeletalMesh(), DataAsset->SkeletalLODIndex, DataAsset->NumDriverTriangles, DataAsset->Sigma);
	}

	// Get Number of Source Vertices (StaticMesh)
	const int32 NumVertices = Mapping.GetNumSourceVertices();
	if (!NumVertices)
	{
		return false;
	}

	// ---------------------------------------------------------------------------
	// Get Reference Skeleton Transforms
	//
	TArray<FVector3f> BoneRefPositions;
	TArray<FVector4f> BoneRefRotations;
	TArray<FVector3f> BonePositions;
	TArray<FVector4f> BoneRotations;
	
	if (DataAsset->Mode == EAnimToTextureMode::Bone)
	{
		// Gets Ref Bone Position and Rotations.
		DataAsset->NumBones = GetRefBonePositionsAndRotations(DataAsset->GetSkeletalMesh(),
			BoneRefPositions, BoneRefRotations);

		// Add RefPose 
		// Note: this is added in the first frame of the Bone Position and Rotation Textures
		BonePositions.Append(BoneRefPositions);
		BoneRotations.Append(BoneRefRotations);
	}

	// --------------------------------------------------------------------------

	// Create Temp Actor
	check(GEditor);
	UWorld* World = GEditor->GetEditorWorldContext().World();
	check(World);

	AActor* Actor = World->SpawnActor<AActor>();
	check(Actor);

	// Create Temp SkeletalMesh Component
	USkeletalMeshComponent* SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(Actor);
	check(SkeletalMeshComponent);
	SkeletalMeshComponent->SetSkeletalMesh(DataAsset->GetSkeletalMesh());
	SkeletalMeshComponent->SetForcedLOD(1); // Force to LOD0;
	SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
	SkeletalMeshComponent->SetUpdateAnimationInEditor(true);
	SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	SkeletalMeshComponent->RegisterComponent();

	// ---------------------------------------------------------------------------
	// Get Vertex Data (for all frames)
	//		
	TArray<FVector3f> VertexDeltas;
	TArray<FVector3f> VertexNormals;
	
	// Get Animation Frames Data
	//
	for (int32 AnimSequenceIndex = 0; AnimSequenceIndex < AnimSequences.Num(); AnimSequenceIndex++)
	{
		const FAnimToTextureAnimSequenceInfo& AnimSequenceInfo = AnimSequences[AnimSequenceIndex];

		// Set Animation
		UAnimSequence* AnimSequence = AnimSequenceInfo.AnimSequence;
		SkeletalMeshComponent->SetAnimation(AnimSequence);

		// Get Number of Frames
		int32 AnimStartFrame;
		int32 AnimEndFrame;
		const int32 AnimNumFrames = GetAnimationFrameRange(AnimSequenceInfo, AnimStartFrame, AnimEndFrame);
		const float AnimStartTime = AnimSequence->GetTimeAtFrame(AnimStartFrame);

		int32 SampleIndex = 0;
		const float SampleInterval = 1.f / DataAsset->SampleRate;

		// Progress Bar
		FFormatNamedArguments Args;
		Args.Add(TEXT("AnimSequenceIndex"), AnimSequenceIndex+1);
		Args.Add(TEXT("NumAnimSequences"), AnimSequences.Num());
		Args.Add(TEXT("AnimSequence"), FText::FromString(*AnimSequence->GetFName().ToString()));
		FScopedSlowTask AnimProgressBar(AnimNumFrames, FText::Format(LOCTEXT("ProcessingAnimSequence", "Processing AnimSequence: {AnimSequence} [{AnimSequenceIndex}/{NumAnimSequences}]"), Args), true /*Enabled*/);
		AnimProgressBar.MakeDialog(false /*bShowCancelButton*/, false /*bAllowInPIE*/);

		while (SampleIndex < AnimNumFrames)
		{
			AnimProgressBar.EnterProgressFrame();

			const float DeltaTime = (float)SampleIndex * SampleInterval;
			const float Time = AnimStartTime + DeltaTime;
			SampleIndex++;

			// Go To Time
			SkeletalMeshComponent->SetPosition(Time);
			// Update SkelMesh Animation.
			SkeletalMeshComponent->TickAnimation(DeltaTime, false/*bNeedsValidRootMotion*/); // fix for evaluating rigid bodies in post process animbp
			SkeletalMeshComponent->RefreshBoneTransforms(nullptr /*TickFunction*/);
			
			// ---------------------------------------------------------------------------
			// Store Vertex Deltas & Normals.
			//
			if (DataAsset->Mode == EAnimToTextureMode::Vertex)
			{
				TArray<FVector3f> VertexFrameDeltas;
				TArray<FVector3f> VertexFrameNormals;
				
				GetVertexDeltasAndNormals(SkeletalMeshComponent, DataAsset->SkeletalLODIndex,
					Mapping, DataAsset->RootTransform,
					VertexFrameDeltas, VertexFrameNormals);
					
				VertexDeltas.Append(VertexFrameDeltas);
				VertexNormals.Append(VertexFrameNormals);
			}

			// ---------------------------------------------------------------------------
			// Store Bone Positions & Rotations
			//
			else if (DataAsset->Mode == EAnimToTextureMode::Bone)
			{
				TArray<FVector3f> BoneFramePositions;
				TArray<FVector4f> BoneFrameRotations;

				GetBonePositionsAndRotations(SkeletalMeshComponent, BoneRefPositions,
					BoneFramePositions, BoneFrameRotations);

				BonePositions.Append(BoneFramePositions);
				BoneRotations.Append(BoneFrameRotations);

			}
		} // End Frame

		// Store Anim Info Data
		FAnimToTextureAnimInfo AnimInfo;
		AnimInfo.StartFrame = DataAsset->NumFrames;
		AnimInfo.EndFrame = DataAsset->NumFrames + AnimNumFrames - 1;
		DataAsset->Animations.Add(AnimInfo);

		// Accumulate Frames
		DataAsset->NumFrames += AnimNumFrames;

	} // End Anim
		
	// Destroy Temp Component & Actor
	SkeletalMeshComponent->UnregisterComponent();
	SkeletalMeshComponent->DestroyComponent();
	Actor->Destroy();
	
	// ---------------------------------------------------------------------------

	if (DataAsset->Mode == EAnimToTextureMode::Vertex)
	{
		// Find Best Resolution for Vertex Data
		int32 Height, Width;
		if (!FindBestResolution(DataAsset->NumFrames, NumVertices, 
								Height, Width, DataAsset->VertexRowsPerFrame, 
								DataAsset->MaxHeight, DataAsset->MaxWidth, DataAsset->bEnforcePowerOfTwo))
		{
			UE_LOGF(LogAnimToTextureEditor, Warning, "Vertex Animation data cannot be fit in a %ix%i texture.", DataAsset->MaxHeight, DataAsset->MaxWidth);
			return false;
		}

		// Normalize Vertex Data
		TArray<FVector3f> NormalizedVertexDeltas;
		TArray<FVector3f> NormalizedVertexNormals;
		NormalizeVertexData(
			VertexDeltas, VertexNormals,
			DataAsset->VertexMinBBox, DataAsset->VertexSizeBBox,
			NormalizedVertexDeltas, NormalizedVertexNormals);

		// Write Textures
		if (DataAsset->Precision == EAnimToTexturePrecision::SixteenBits)
		{
			WriteVectorsToTexture<FVector3f, FHighPrecision>(NormalizedVertexDeltas, DataAsset->NumFrames, DataAsset->VertexRowsPerFrame, Height, Width, DataAsset->GetVertexPositionTexture());
			WriteVectorsToTexture<FVector3f, FHighPrecision>(NormalizedVertexNormals, DataAsset->NumFrames, DataAsset->VertexRowsPerFrame, Height, Width, DataAsset->GetVertexNormalTexture());
		}
		else
		{
			WriteVectorsToTexture<FVector3f, FLowPrecision>(NormalizedVertexDeltas, DataAsset->NumFrames, DataAsset->VertexRowsPerFrame, Height, Width, DataAsset->GetVertexPositionTexture());
			WriteVectorsToTexture<FVector3f, FLowPrecision>(NormalizedVertexNormals, DataAsset->NumFrames, DataAsset->VertexRowsPerFrame, Height, Width, DataAsset->GetVertexNormalTexture());
		}		

		// Add Vertex UVChannel
		CreateUVChannel(DataAsset->GetStaticMesh(), DataAsset->StaticLODIndex, DataAsset->UVChannel, Height, Width);

		// Update Bounds
		SetBoundsExtensions(DataAsset->GetStaticMesh(), (FVector)DataAsset->VertexMinBBox, (FVector)DataAsset->VertexSizeBBox);

		// Done with StaticMesh
		DataAsset->GetStaticMesh()->PostEditChange();
	}

	// ---------------------------------------------------------------------------
	
	if (DataAsset->Mode == EAnimToTextureMode::Bone)
	{
		// Find Best Resolution for Bone Data
		int32 Height, Width;

		// Write Bone Position and Rotation Textures
		{
			// Note we are adding +1 frame for the ref pose
			if (!FindBestResolution(DataAsset->NumFrames + 1, DataAsset->NumBones,
				Height, Width, DataAsset->BoneRowsPerFrame,
				DataAsset->MaxHeight, DataAsset->MaxWidth, DataAsset->bEnforcePowerOfTwo))
			{
				UE_LOGF(LogAnimToTextureEditor, Warning, "Bone Animation data cannot be fit in a %ix%i texture.", DataAsset->MaxHeight, DataAsset->MaxWidth);
				return false;
			}

			// Normalize Bone Data
			TArray<FVector3f> NormalizedBonePositions;
			TArray<FVector4f> NormalizedBoneRotations;
			NormalizeBoneData(
				BonePositions, BoneRotations,
				DataAsset->BoneMinBBox, DataAsset->BoneSizeBBox,
				NormalizedBonePositions, NormalizedBoneRotations);

			// Write Textures
			if (DataAsset->Precision == EAnimToTexturePrecision::SixteenBits)
			{
				WriteVectorsToTexture<FVector3f, FHighPrecision>(NormalizedBonePositions, DataAsset->NumFrames + 1, DataAsset->BoneRowsPerFrame, Height, Width, DataAsset->GetBonePositionTexture());
				WriteVectorsToTexture<FVector4f, FHighPrecision>(NormalizedBoneRotations, DataAsset->NumFrames + 1, DataAsset->BoneRowsPerFrame, Height, Width, DataAsset->GetBoneRotationTexture());
			}
			else
			{
				WriteVectorsToTexture<FVector3f, FLowPrecision>(NormalizedBonePositions, DataAsset->NumFrames + 1, DataAsset->BoneRowsPerFrame, Height, Width, DataAsset->GetBonePositionTexture());
				WriteVectorsToTexture<FVector4f, FLowPrecision>(NormalizedBoneRotations, DataAsset->NumFrames + 1, DataAsset->BoneRowsPerFrame, Height, Width, DataAsset->GetBoneRotationTexture());
			}

			// Update Bounds
			SetBoundsExtensions(DataAsset->GetStaticMesh(), (FVector)DataAsset->BoneMinBBox, (FVector)DataAsset->BoneSizeBBox);
		}

		// ---------------------------------------------------------------------------
		
		// Write Weights Texture
		{
			// Find Best Resolution for Bone Weights Texture
			if (!FindBestResolution(2, NumVertices,
				Height, Width, DataAsset->BoneWeightRowsPerFrame,
				DataAsset->MaxHeight, DataAsset->MaxWidth, DataAsset->bEnforcePowerOfTwo))
			{
				UE_LOGF(LogAnimToTextureEditor, Warning, "Weights Data cannot be fit in a %ix%i texture.", DataAsset->MaxHeight, DataAsset->MaxWidth);
				return false;
			}

			TArray<TVertexSkinWeight<4>> SkinWeights;

			// Reduce BoneWeights to 4 Influences.
			if (SocketIndex == INDEX_NONE)
			{
				// Project SkinWeights from SkeletalMesh to StaticMesh
				TArray<VertexSkinWeightMax> StaticMeshSkinWeights;
				Mapping.ProjectSkinWeights(StaticMeshSkinWeights);

				// Reduce Weights to 4 highest influences.
				ReduceSkinWeights(StaticMeshSkinWeights, SkinWeights);
			}
			// If Valid Socket, set all influences to same index.
			else
			{
				// Set all indices and weights to same SocketIndex
				SkinWeights.SetNumUninitialized(NumVertices);
				for (TVertexSkinWeight<4>& SkinWeight : SkinWeights)
				{
					SkinWeight.BoneWeights = TStaticArray<uint8, 4>(InPlace, 255);
					SkinWeight.MeshBoneIndices = TStaticArray<uint16, 4>(InPlace, SocketIndex);
				}
			}

			// Write Bone Weights Texture
			if (DataAsset->Precision == EAnimToTexturePrecision::SixteenBits)
			{
				WriteSkinWeightsToTexture<FHighPrecision>(SkinWeights, DataAsset->NumBones,
					DataAsset->BoneWeightRowsPerFrame, Height, Width, DataAsset->GetBoneWeightTexture());
			}
			else
			{
				WriteSkinWeightsToTexture<FLowPrecision>(SkinWeights, DataAsset->NumBones,
					DataAsset->BoneWeightRowsPerFrame, Height, Width, DataAsset->GetBoneWeightTexture());
			}

			// Add Vertex UVChannel
			CreateUVChannel(DataAsset->GetStaticMesh(), DataAsset->StaticLODIndex, DataAsset->UVChannel, Height, Width);
		}

		// Done with StaticMesh
		DataAsset->GetStaticMesh()->PostEditChange();
	}

	// ---------------------------------------------------------------------------
	// Mark Packages dirty
	//
	DataAsset->MarkPackageDirty();
	
	// All good here !
	return true;
}


bool UAnimToTextureBPLibrary::CheckDataAsset(const UAnimToTextureDataAsset* DataAsset, 
	int32& OutSocketIndex, TArray<FAnimToTextureAnimSequenceInfo>& OutAnimSequences)
{
	// Check StaticMesh
	if (!DataAsset->GetStaticMesh())
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid StaticMesh");
		return false;
	}

	// Check SkeletalMesh
	if (!DataAsset->GetSkeletalMesh())
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid SkeletalMesh");
		return false;
	}

	// Check Skeleton
	if (!DataAsset->GetSkeletalMesh()->GetSkeleton())
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid SkeletalMesh. No valid Skeleton found");
		return false;
	}

	// Check StaticMesh LOD
	if (!DataAsset->GetStaticMesh()->IsSourceModelValid(DataAsset->StaticLODIndex))
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid StaticMesh LOD Index: %i", DataAsset->StaticLODIndex);
		return false;
	}

	// Check SkeletalMesh LOD
	if (!DataAsset->GetSkeletalMesh()->IsValidLODIndex(DataAsset->SkeletalLODIndex))
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid SkeletalMesh LOD Index: %i", DataAsset->SkeletalLODIndex);
		return false;
	}

	// Check Socket.
	OutSocketIndex = INDEX_NONE;
	if (DataAsset->AttachToSocket.IsValid() && !DataAsset->AttachToSocket.IsNone())
	{
		// Get Bone Names (no virtual)
		TArray<FName> BoneNames;
		GetBoneNames(DataAsset->GetSkeletalMesh(), BoneNames);

		// Check if Socket is in BoneNames
		if (!BoneNames.Find(DataAsset->AttachToSocket, OutSocketIndex))
		{
			UE_LOGF(LogAnimToTextureEditor, Warning, "Socket: %ls not found in Raw Bone List", *DataAsset->AttachToSocket.ToString());
			return false;
		}
		else
		{
			// TODO: SocketIndex can only be < TNumericLimits<uint16>::Max()
		}
	}

	// Check if UVChannel is being used by the Lightmap UV
	const FStaticMeshSourceModel& SourceModel = DataAsset->GetStaticMesh()->GetSourceModel(DataAsset->StaticLODIndex);
	if (SourceModel.BuildSettings.bGenerateLightmapUVs &&
		SourceModel.BuildSettings.DstLightmapIndex == DataAsset->UVChannel)
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid StaticMesh UVChannel: %i. Already used by LightMap", DataAsset->UVChannel);
		return false;
	}

	// Check if NumBones > 256
	const int32 NumBones = GetNumBones(DataAsset->GetSkeletalMesh());
	if (DataAsset->Precision == EAnimToTexturePrecision::EightBits &&
		NumBones > 256)
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Too many Bones: %i. There is a maximum of 256 bones for 8bit Precision", NumBones);
		return false;
	}
	
	// Check Animations
	OutAnimSequences.Reset();
	for (const FAnimToTextureAnimSequenceInfo& AnimSequenceInfo : DataAsset->AnimSequences)
	{
		const UAnimSequence* AnimSequence = AnimSequenceInfo.AnimSequence;

		if (AnimSequenceInfo.bEnabled && AnimSequence)
		{
			// Make sure SkeletalMesh is compatible with AnimSequence
			if (!DataAsset->GetSkeletalMesh()->GetSkeleton()->IsCompatibleForEditor(AnimSequence->GetSkeleton()))
			{
				UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid AnimSequence: %ls for given SkeletalMesh: %ls", *AnimSequence->GetFName().ToString(), *DataAsset->GetSkeletalMesh()->GetFName().ToString());
				return false;
			}

			// Check Frame Range
			if (AnimSequenceInfo.bUseCustomRange &&
				(AnimSequenceInfo.StartFrame < 0 ||
					AnimSequenceInfo.EndFrame > AnimSequence->GetNumberOfSampledKeys() - 1 ||
					AnimSequenceInfo.EndFrame - AnimSequenceInfo.StartFrame < 0))
			{
				UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid CustomRange for AnimSequence: %ls", *AnimSequence->GetName());
				return false;
			}

			// Store Valid AnimSequenceInfo
			OutAnimSequences.Add(AnimSequenceInfo);
		}
	}

	if (!OutAnimSequences.Num())
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "No valid AnimSequences found");
		return false;
	}

	// All Good !
	return true;
}

int32 UAnimToTextureBPLibrary::GetAnimationFrameRange(const FAnimToTextureAnimSequenceInfo& Animation, int32& OutStartFrame, int32& OutEndFrame)
{
	if (!Animation.AnimSequence)
	{
		return INDEX_NONE;
	}

	// Get Range from AnimSequence
	if (!Animation.bUseCustomRange)
	{
		OutStartFrame = 0;
		OutEndFrame = Animation.AnimSequence->GetNumberOfSampledKeys() - 1; // AnimSequence->GetNumberOfFrames();
	}
	// Get Custom Range
	else
	{
		OutStartFrame = Animation.StartFrame;
		OutEndFrame = Animation.EndFrame;
	}

	// Return Number of Frames
	return OutEndFrame - OutStartFrame + 1;
}

// 
void UAnimToTextureBPLibrary::GetVertexDeltasAndNormals(const USkeletalMeshComponent* SkeletalMeshComponent, const int32 LODIndex, 
	const AnimToTexture_Private::FSourceMeshToDriverMesh& SourceMeshToDriverMesh,
	const FTransform RootTransform,
	TArray<FVector3f>& OutVertexDeltas, TArray<FVector3f>& OutVertexNormals)
{
	OutVertexDeltas.Reset();
	OutVertexNormals.Reset();
		
	// Get Deformed vertices at current frame
	TArray<FVector3f> SkinnedVertices;
	GetSkinnedVertices(SkeletalMeshComponent, LODIndex, SkinnedVertices);
	
	// Get Source Vertices (StaticMesh)
	TArray<FVector3f> SourceVertices;
	const int32 NumVertices = SourceMeshToDriverMesh.GetSourceVertices(SourceVertices);

	// Deform Source Vertices with DriverMesh (SkeletalMesh
	TArray<FVector3f> DeformedVertices;
	TArray<FVector3f> DeformedNormals;
	SourceMeshToDriverMesh.DeformVerticesAndNormals(SkinnedVertices, DeformedVertices, DeformedNormals);

	// Allocate
	check(DeformedVertices.Num() == NumVertices && DeformedNormals.Num() == NumVertices);
	OutVertexDeltas.SetNumUninitialized(NumVertices);
	OutVertexNormals.SetNumUninitialized(NumVertices);

	// Transform Vertices and Normals with RootTransform
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; VertexIndex++)
	{
		const FVector3f& SourceVertex   = SourceVertices[VertexIndex];
		const FVector3f& DeformedVertex = DeformedVertices[VertexIndex];
		const FVector3f& DeformedNormal = DeformedNormals[VertexIndex];
	
		// Transform Position and Delta with RootTransform
		const FVector3f TransformedVertexDelta = ((FVector3f)RootTransform.TransformPosition((FVector)DeformedVertex)) - SourceVertex;
		const FVector3f TransformedVertexNormal = (FVector3f)RootTransform.TransformVector((FVector)DeformedNormal);
		
		OutVertexDeltas[VertexIndex] = TransformedVertexDelta;
		OutVertexNormals[VertexIndex] = TransformedVertexNormal;
	}
}


int32 UAnimToTextureBPLibrary::GetRefBonePositionsAndRotations(const USkeletalMesh* SkeletalMesh,
	TArray<FVector3f>& OutBoneRefPositions, TArray<FVector4f>& OutBoneRefRotations)
{
	check(SkeletalMesh);

	OutBoneRefPositions.Reset();
	OutBoneRefRotations.Reset();

	// Get Number of RawBones (no virtual)
	const int32 NumBones = GetNumBones(SkeletalMesh);
	
	// Get Raw Ref Bone (no virtual)
	TArray<FTransform> RefBoneTransforms;
	GetRefBoneTransforms(SkeletalMesh, RefBoneTransforms);
	DecomposeTransformations(RefBoneTransforms, OutBoneRefPositions, OutBoneRefRotations);

	return NumBones;
}


int32 UAnimToTextureBPLibrary::GetBonePositionsAndRotations(const USkeletalMeshComponent* SkeletalMeshComponent, const TArray<FVector3f>& BoneRefPositions,
	TArray<FVector3f>& BonePositions, TArray<FVector4f>& BoneRotations)
{
	check(SkeletalMeshComponent);

	BonePositions.Reset();
	BoneRotations.Reset();

	// Get Relative Transforms
	// Note: Size is of Raw bones in SkeletalMesh. These are the original/raw bones of the asset, without Virtual Bones.
	TArray<FMatrix44f> RefToLocals;
	SkeletalMeshComponent->CacheRefToLocalMatrices(RefToLocals);
	const int32 NumBones = RefToLocals.Num();

	// check size
	check(NumBones == BoneRefPositions.Num());

	// Get Component Space Transforms
	// Note returns all transforms, including VirtualBones
	const TArray<FTransform>& CompSpaceTransforms = SkeletalMeshComponent->GetComponentSpaceTransforms();
	check(CompSpaceTransforms.Num() >= RefToLocals.Num());

	// Allocate
	BonePositions.SetNumUninitialized(NumBones);
	BoneRotations.SetNumUninitialized(NumBones);

	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		// Decompose Transformation (ComponentSpace)
		const FTransform& CompSpaceTransform = CompSpaceTransforms[BoneIndex];
		FVector3f BonePosition;
		FVector4f BoneRotation;
		DecomposeTransformation(CompSpaceTransform, BonePosition, BoneRotation);

		// Position Delta (from RefPose)
		const FVector3f Delta = BonePosition - BoneRefPositions[BoneIndex];

		// Decompose Transformation (Relative to RefPose)
		FVector3f BoneRelativePosition;
		FVector4f BoneRelativeRotation;
		const FMatrix RefToLocalMatrix(RefToLocals[BoneIndex]);
		const FTransform RelativeTransform(RefToLocalMatrix);
		DecomposeTransformation(RelativeTransform, BoneRelativePosition, BoneRelativeRotation);

		BonePositions[BoneIndex] = Delta;
		BoneRotations[BoneIndex] = BoneRelativeRotation;
	}

	return NumBones;
}


void UAnimToTextureBPLibrary::UpdateMaterialInstanceFromDataAsset(const UAnimToTextureDataAsset* DataAsset, UMaterialInstanceConstant* MaterialInstance, 
	const EMaterialParameterAssociation MaterialParameterAssociation)
{
	check(DataAsset);
	check(MaterialInstance);
	
	// Set UVChannel
	switch (DataAsset->UVChannel)
	{
		case 0:
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV0, true, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV1, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV2, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV3, false, MaterialParameterAssociation);
			break;
		case 1:
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV0, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV1, true, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV2, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV3, false, MaterialParameterAssociation);
			break;
		case 2:
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV0, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV1, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV2, true, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV3, false, MaterialParameterAssociation);
			break;
		case 3:
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV0, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV1, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV2, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV3, true, MaterialParameterAssociation);
			break;
		default:
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV0, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV1, true, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV2, false, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseUV3, false, MaterialParameterAssociation);
			break;
	}

	// Update Vertex Params
	if (DataAsset->Mode == EAnimToTextureMode::Vertex)
	{
		UMaterialEditingLibrary::SetMaterialInstanceVectorParameterValue(MaterialInstance, AnimToTextureParamNames::MinBBox, FLinearColor(DataAsset->VertexMinBBox), MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceVectorParameterValue(MaterialInstance, AnimToTextureParamNames::SizeBBox, FLinearColor(DataAsset->VertexSizeBBox), MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::RowsPerFrame, DataAsset->VertexRowsPerFrame, MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(MaterialInstance, AnimToTextureParamNames::VertexPositionTexture, DataAsset->GetVertexPositionTexture(), MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(MaterialInstance, AnimToTextureParamNames::VertexNormalTexture, DataAsset->GetVertexNormalTexture(), MaterialParameterAssociation);
	}

	// Update Bone Params
	else if (DataAsset->Mode == EAnimToTextureMode::Bone)
	{
		UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::NumBones, DataAsset->NumBones, MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceVectorParameterValue(MaterialInstance, AnimToTextureParamNames::MinBBox, FLinearColor(DataAsset->BoneMinBBox), MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceVectorParameterValue(MaterialInstance, AnimToTextureParamNames::SizeBBox, FLinearColor(DataAsset->BoneSizeBBox), MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::RowsPerFrame, DataAsset->BoneRowsPerFrame, MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::BoneWeightRowsPerFrame, DataAsset->BoneWeightRowsPerFrame, MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(MaterialInstance, AnimToTextureParamNames::BonePositionTexture, DataAsset->GetBonePositionTexture(), MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(MaterialInstance, AnimToTextureParamNames::BoneRotationTexture, DataAsset->GetBoneRotationTexture(), MaterialParameterAssociation);
		UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(MaterialInstance, AnimToTextureParamNames::BoneWeightsTexture, DataAsset->GetBoneWeightTexture(), MaterialParameterAssociation);

		// Num Influences
		switch (DataAsset->NumBoneInfluences)
		{
			case EAnimToTextureNumBoneInfluences::One:
				UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseTwoInfluences, false, MaterialParameterAssociation);
				UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseFourInfluences, false, MaterialParameterAssociation);
				break;
			case EAnimToTextureNumBoneInfluences::Two:
				UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseTwoInfluences, true, MaterialParameterAssociation);
				UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseFourInfluences, false, MaterialParameterAssociation);
				break;
			case EAnimToTextureNumBoneInfluences::Four:
				UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseTwoInfluences, false, MaterialParameterAssociation);
				UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::UseFourInfluences, true, MaterialParameterAssociation);
				break;
		}
	}

	// AutoPlay
	UMaterialEditingLibrary::SetMaterialInstanceStaticSwitchParameterValue(MaterialInstance, AnimToTextureParamNames::AutoPlay, DataAsset->bAutoPlay, MaterialParameterAssociation);
	if (DataAsset->bAutoPlay)
	{
		if (DataAsset->Animations.IsValidIndex(DataAsset->AnimationIndex))
		{
			UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::StartFrame, DataAsset->Animations[DataAsset->AnimationIndex].StartFrame, MaterialParameterAssociation);
			UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::EndFrame, DataAsset->Animations[DataAsset->AnimationIndex].EndFrame, MaterialParameterAssociation);
		}
		else
		{
			UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid AnimationIndex: %i", DataAsset->AnimationIndex);
		}
	}
	else
	{
		if (DataAsset->Frame >= 0 && DataAsset->Frame < DataAsset->NumFrames)
		{
			UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::Frame, DataAsset->Frame, MaterialParameterAssociation);
		}
		else
		{
			UE_LOGF(LogAnimToTextureEditor, Warning, "Frame out of range: %i", DataAsset->Frame);
		}
	}
	
	// NumFrames
	UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::NumFrames, DataAsset->NumFrames, MaterialParameterAssociation);

	// SampleRate
	UMaterialEditingLibrary::SetMaterialInstanceScalarParameterValue(MaterialInstance, AnimToTextureParamNames::SampleRate, DataAsset->SampleRate, MaterialParameterAssociation);

	// Update Material
	UMaterialEditingLibrary::UpdateMaterialInstance(MaterialInstance);

	// Rebuild Material
	UMaterialEditingLibrary::RebuildMaterialInstanceEditors(MaterialInstance->GetMaterial());

	// Set Preview Mesh
	if(UStaticMesh* StaticMesh = DataAsset->GetStaticMesh())
	{
		MaterialInstance->PreviewMesh = FSoftObjectPath(StaticMesh);
	}

	MaterialInstance->MarkPackageDirty();

}

bool UAnimToTextureBPLibrary::SetLightMapIndex(UStaticMesh* StaticMesh, const int32 LODIndex, const int32 LightmapIndex, bool bGenerateLightmapUVs)
{
	check(StaticMesh);

	if (!StaticMesh->IsSourceModelValid(LODIndex))
	{
		return false;
	}

	for (int32 Index=0; Index < LightmapIndex; Index++)
	{
		if (LightmapIndex > StaticMesh->GetNumUVChannels(LODIndex))
		{
			StaticMesh->AddUVChannel(LODIndex);
		}
	}

	// Set Build Settings
	FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LODIndex);
	SourceModel.BuildSettings.bGenerateLightmapUVs = bGenerateLightmapUVs;
	SourceModel.BuildSettings.DstLightmapIndex = LightmapIndex;
	StaticMesh->SetLightMapCoordinateIndex(LightmapIndex);

	// Build Mesh
	StaticMesh->Build(false);
	StaticMesh->PostEditChange();
	StaticMesh->MarkPackageDirty();

	return true;
}


UStaticMesh* UAnimToTextureBPLibrary::ConvertSkeletalMeshToStaticMesh(USkeletalMesh* SkeletalMesh, const FString PackageName, const int32 LODIndex)
{
	check(SkeletalMesh);

	if (PackageName.IsEmpty() || !FPackageName::IsValidObjectPath(PackageName))
	{
		return nullptr;
	}

	if (!SkeletalMesh->IsValidLODIndex(LODIndex))
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid LODIndex: %i", LODIndex);
		return nullptr;
	}

	// Create Temp Actor
	check(GEditor);
	UWorld* World = GEditor->GetEditorWorldContext().World();
	check(World);
	AActor* Actor = World->SpawnActor<AActor>();
	check(Actor);

	// Create Temp SkeletalMesh Component
	USkeletalMeshComponent* MeshComponent = NewObject<USkeletalMeshComponent>(Actor);
	MeshComponent->RegisterComponent();
	MeshComponent->SetSkeletalMesh(SkeletalMesh);
	TArray<UMeshComponent*> MeshComponents = { MeshComponent };

	UStaticMesh* OutStaticMesh = nullptr;
	bool bGeneratedCorrectly = true;

	// Create New StaticMesh
	if (!FPackageName::DoesPackageExist(PackageName))
	{
		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		OutStaticMesh = MeshUtilities.ConvertMeshesToStaticMesh(MeshComponents, FTransform::Identity, PackageName);
	}
	// Update Existing StaticMesh
	else
	{
		// Load Existing Mesh
		OutStaticMesh = LoadObject<UStaticMesh>(nullptr, *PackageName);
	}

	if (OutStaticMesh)
	{
		// Create Temp Package.
		// because 
		UPackage* TransientPackage = GetTransientPackage();

		// Create Temp Mesh.
		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		UStaticMesh* TempMesh = MeshUtilities.ConvertMeshesToStaticMesh(MeshComponents, FTransform::Identity, TransientPackage->GetPathName());

		// make sure transactional flag is on
		TempMesh->SetFlags(RF_Transactional);

		// Copy All LODs
		if (LODIndex < 0)
		{
			const int32 NumSourceModels = TempMesh->GetNumSourceModels();
			OutStaticMesh->SetNumSourceModels(NumSourceModels);

			for (int32 Index = 0; Index < NumSourceModels; ++Index)
			{
				// Get RawMesh
				FRawMesh RawMesh;
				TempMesh->GetSourceModel(Index).LoadRawMesh(RawMesh);

				// Set RawMesh
				OutStaticMesh->GetSourceModel(Index).SaveRawMesh(RawMesh);
			};
		}

		// Copy Single LOD
		else
		{
			if (LODIndex >= TempMesh->GetNumSourceModels())
			{
				UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid Source Model Index: %i", LODIndex);
				bGeneratedCorrectly = false;
			}
			else
			{
				OutStaticMesh->SetNumSourceModels(1);

				// Get RawMesh
				FRawMesh RawMesh;
				TempMesh->GetSourceModel(LODIndex).LoadRawMesh(RawMesh);

				// Set RawMesh
				OutStaticMesh->GetSourceModel(0).SaveRawMesh(RawMesh);
			}
		}
			
		// Copy Materials
		const TArray<FStaticMaterial>& Materials = TempMesh->GetStaticMaterials();
		OutStaticMesh->SetStaticMaterials(Materials);

		// Done
		TArray<FText> OutErrors;
		OutStaticMesh->Build(true, &OutErrors);
		OutStaticMesh->MarkPackageDirty();
	}

	// Destroy Temp Component and Actor
	MeshComponent->UnregisterComponent();
	MeshComponent->DestroyComponent();
	Actor->Destroy();

	return bGeneratedCorrectly ? OutStaticMesh : nullptr;
}


void UAnimToTextureBPLibrary::NormalizeVertexData(
	const TArray<FVector3f>& Deltas, const TArray<FVector3f>& Normals,
	FVector3f& OutMinBBox, FVector3f& OutSizeBBox,
	TArray<FVector3f>& OutNormalizedDeltas, TArray<FVector3f>& OutNormalizedNormals)
{
	check(Deltas.Num() == Normals.Num());

	// ---------------------------------------------------------------------------
	// Compute Bounding Box
	//
	OutMinBBox = { TNumericLimits<float>::Max(), TNumericLimits<float>::Max(), TNumericLimits<float>::Max() };
	FVector3f MaxBBox = { TNumericLimits<float>::Min(), TNumericLimits<float>::Min(), TNumericLimits<float>::Min() };
	
	for (const FVector3f& Delta: Deltas)
	{
		// Find Min/Max BoundingBox
		OutMinBBox.X = FMath::Min(Delta.X, OutMinBBox.X);
		OutMinBBox.Y = FMath::Min(Delta.Y, OutMinBBox.Y);
		OutMinBBox.Z = FMath::Min(Delta.Z, OutMinBBox.Z);

		MaxBBox.X = FMath::Max(Delta.X, MaxBBox.X);
		MaxBBox.Y = FMath::Max(Delta.Y, MaxBBox.Y);
		MaxBBox.Z = FMath::Max(Delta.Z, MaxBBox.Z);
	}

	OutSizeBBox = MaxBBox - OutMinBBox;

	// ---------------------------------------------------------------------------
	// Normalize Vertex Position Deltas
	// Basically we want all deltas to be between [0, 1]
	
	// Compute Normalization Factor per-axis.
	const FVector3f NormFactor = {
		1.f / static_cast<float>(OutSizeBBox.X),
		1.f / static_cast<float>(OutSizeBBox.Y),
		1.f / static_cast<float>(OutSizeBBox.Z) };

	OutNormalizedDeltas.SetNumUninitialized(Deltas.Num());
	for (int32 Index = 0; Index < Deltas.Num(); ++Index)
	{
		OutNormalizedDeltas[Index] = (Deltas[Index] - OutMinBBox) * NormFactor;
	}

	// ---------------------------------------------------------------------------
	// Normalize Vertex Normals
	// And move them to [0, 1]
	
	OutNormalizedNormals.SetNumUninitialized(Normals.Num());
	for (int32 Index = 0; Index < Normals.Num(); ++Index)
	{
		OutNormalizedNormals[Index] = (Normals[Index].GetSafeNormal() + FVector3f::OneVector) * 0.5f;
	}

}

void UAnimToTextureBPLibrary::NormalizeBoneData(
	const TArray<FVector3f>& Positions, const TArray<FVector4f>& Rotations,
	FVector3f& OutMinBBox, FVector3f& OutSizeBBox, 
	TArray<FVector3f>& OutNormalizedPositions, TArray<FVector4f>& OutNormalizedRotations)
{
	check(Positions.Num() == Rotations.Num());

	// ---------------------------------------------------------------------------
	// Compute Position Bounding Box
	//
	OutMinBBox = { TNumericLimits<float>::Max(), TNumericLimits<float>::Max(), TNumericLimits<float>::Max() };
	FVector3f MaxBBox = { TNumericLimits<float>::Min(), TNumericLimits<float>::Min(), TNumericLimits<float>::Min() };

	for (const FVector3f& Position : Positions)
	{
		// Find Min/Max BoundingBox
		OutMinBBox.X = FMath::Min(Position.X, OutMinBBox.X);
		OutMinBBox.Y = FMath::Min(Position.Y, OutMinBBox.Y);
		OutMinBBox.Z = FMath::Min(Position.Z, OutMinBBox.Z);

		MaxBBox.X = FMath::Max(Position.X, MaxBBox.X);
		MaxBBox.Y = FMath::Max(Position.Y, MaxBBox.Y);
		MaxBBox.Z = FMath::Max(Position.Z, MaxBBox.Z);
	}

	OutSizeBBox = MaxBBox - OutMinBBox;

	// ---------------------------------------------------------------------------
	// Normalize Bone Position.
	// Basically we want all positions to be between [0, 1]

	// Compute Normalization Factor per-axis.
	const FVector3f NormFactor = {
		1.f / static_cast<float>(OutSizeBBox.X),
		1.f / static_cast<float>(OutSizeBBox.Y),
		1.f / static_cast<float>(OutSizeBBox.Z) };

	OutNormalizedPositions.SetNumUninitialized(Positions.Num());
	for (int32 Index = 0; Index < Positions.Num(); ++Index)
	{
		OutNormalizedPositions[Index] = (Positions[Index] - OutMinBBox) * NormFactor;
	}

	// ---------------------------------------------------------------------------
	// Normalize Rotations
	// And move them to [0, 1]
	OutNormalizedRotations.SetNumUninitialized(Rotations.Num());
	for (int32 Index = 0; Index < Rotations.Num(); ++Index)
	{
		const FVector4f Axis = Rotations[Index];
		const float Angle = Rotations[Index].W; // Angle are returned in radians and they go from [0-pi*2]

		OutNormalizedRotations[Index] = (Axis.GetSafeNormal() + FVector3f::OneVector) * 0.5f;
		OutNormalizedRotations[Index].W = Angle / (PI * 2.f);
	}
}

bool UAnimToTextureBPLibrary::CreateUVChannel(
	UStaticMesh* StaticMesh, const int32 LODIndex, const int32 UVChannelIndex,
	const int32 Height, const int32 Width)
{
	check(StaticMesh);

	if (!StaticMesh->IsSourceModelValid(LODIndex))
	{
		return false;
	}

	// ----------------------------------------------------------------------------
	// Get Mesh Description.
	// This is needed for Inserting UVChannel
	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);
	check(MeshDescription);

	// Add New UVChannel.
	if (UVChannelIndex == StaticMesh->GetNumUVChannels(LODIndex))
	{
		if (!StaticMesh->InsertUVChannel(LODIndex, UVChannelIndex))
		{
			UE_LOGF(LogAnimToTextureEditor, Warning, "Unable to Add UVChannel");
			return false;
		}
	}
	else if (UVChannelIndex > StaticMesh->GetNumUVChannels(LODIndex))
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "UVChannel: %i Out of Range. Number of existing UVChannels: %i", UVChannelIndex, StaticMesh->GetNumUVChannels(LODIndex));
		return false;
	}

	// -----------------------------------------------------------------------------

	TMap<FVertexInstanceID, FVector2D> TexCoords;

	for (const FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
	{
		const FVertexID VertexID = MeshDescription->GetVertexInstanceVertex(VertexInstanceID);
		const int32 VertexIndex = VertexID.GetValue();

		float U = (0.5f / (float)Width) + (VertexIndex % Width) / (float)Width;
		float V = (0.5f / (float)Height) + (VertexIndex / Width) / (float)Height;
		
		TexCoords.Add(VertexInstanceID, FVector2D(U, V));
	}

	// Set Full Precision UVs
	SetFullPrecisionUVs(StaticMesh, LODIndex, true);

	// Set UVs
	if (StaticMesh->SetUVChannel(LODIndex, UVChannelIndex, TexCoords))
	{
		return true;
	}
	else
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Unable to Set UVChannel: %i. TexCoords: %i", UVChannelIndex, TexCoords.Num());
		return false;
	}
}

bool UAnimToTextureBPLibrary::FindBestResolution(
	const int32 NumFrames, const int32 NumElements,
	int32& OutHeight, int32& OutWidth, int32& OutRowsPerFrame,
	const int32 MaxHeight, const int32 MaxWidth, bool bEnforcePowerOfTwo)
{
	if (bEnforcePowerOfTwo)
	{
		OutWidth = 2;
		while (OutWidth < NumElements && OutWidth < MaxWidth)
		{
			OutWidth *= 2;
		}
		OutRowsPerFrame = FMath::CeilToInt(NumElements / (float)OutWidth);

		const int32 TargetHeight = NumFrames * OutRowsPerFrame;
		OutHeight = 2;
		while (OutHeight < TargetHeight)
		{
			OutHeight *= 2;
		}
	}
	else
	{
		OutRowsPerFrame = FMath::CeilToInt(NumElements / (float)MaxWidth);
		OutWidth = FMath::CeilToInt(NumElements / (float)OutRowsPerFrame);
		OutHeight = NumFrames * OutRowsPerFrame;
	}

	const bool bValidResolution = OutWidth <= MaxWidth && OutHeight <= MaxHeight;
	return bValidResolution;
};

void UAnimToTextureBPLibrary::SetFullPrecisionUVs(UStaticMesh* StaticMesh, int32 LODIndex, bool bFullPrecision)
{
	check(StaticMesh);

	if (StaticMesh->IsSourceModelValid(LODIndex))
	{
		FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LODIndex);
		SourceModel.BuildSettings.bUseFullPrecisionUVs = bFullPrecision;
	}
}

void UAnimToTextureBPLibrary::SetBoundsExtensions(UStaticMesh* StaticMesh, const FVector& MinBBox, const FVector& SizeBBox)
{
	check(StaticMesh);

	// Calculate MaxBBox
	const FVector MaxBBox = SizeBBox + MinBBox;

	// Reset current extension bounds
	const FVector PositiveBoundsExtension = StaticMesh->GetPositiveBoundsExtension();
	const FVector NegativeBoundsExtension = StaticMesh->GetNegativeBoundsExtension();
		
	// Get current BoundingBox including extensions
	FBox BoundingBox = StaticMesh->GetBoundingBox();
		
	// Remove extensions from BoundingBox
	BoundingBox.Max -= PositiveBoundsExtension;
	BoundingBox.Min += NegativeBoundsExtension;
		
	// Calculate New BoundingBox
	FVector NewMaxBBox(
		FMath::Max(BoundingBox.Max.X, MaxBBox.X),
		FMath::Max(BoundingBox.Max.Y, MaxBBox.Y),
		FMath::Max(BoundingBox.Max.Z, MaxBBox.Z)
	);
		
	FVector NewMinBBox(
		FMath::Min(BoundingBox.Min.X, MinBBox.X),
		FMath::Min(BoundingBox.Min.Y, MinBBox.Y),
		FMath::Min(BoundingBox.Min.Z, MinBBox.Z)
	);

	// Calculate New Extensions
	FVector NewPositiveBoundsExtension = NewMaxBBox - BoundingBox.Max;
	FVector NewNegativeBoundsExtension = BoundingBox.Min - NewMinBBox;
				
	// Update StaticMesh
	StaticMesh->SetPositiveBoundsExtension(NewPositiveBoundsExtension);
	StaticMesh->SetNegativeBoundsExtension(NewNegativeBoundsExtension);
	StaticMesh->CalculateExtendedBounds();
}

#undef LOCTEXT_NAMESPACE
