// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimToTextureEditorModule.h"

#include "AssetToolsModule.h"
#include "MessageLogModule.h"

#define LOCTEXT_NAMESPACE "AnimToTextureEditor"

DEFINE_LOG_CATEGORY(LogAnimToTextureEditor);

void FAnimToTextureEditorModule::StartupModule()
{
	// Register Log
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = false;
	InitOptions.bAllowClear = true;
	MessageLogModule.RegisterLogListing("AnimToTextureLog", LOCTEXT("AnimToTextureLog", "AnimToTexture Log"), InitOptions);
}

void FAnimToTextureEditorModule::ShutdownModule()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.UnregisterLogListing("AnimToTextureLog");
}

IMPLEMENT_MODULE(FAnimToTextureEditorModule, AnimToTextureEditor)

#undef LOCTEXT_NAMESPACE
