// Copyright Epic Games, Inc. All Rights Reserved.
#include "AnimToTextureMeshMapping.h"
#include "Runtime/Core/Public/Async/ParallelFor.h"

namespace AnimToTexture_Private
{

void FSourceVertexData::Update(const FVector3f& SourceVertex,
	const TArray<FVector3f>& DriverVertices, const TArray<FIntVector3>& DriverTriangles, const TArray<VertexSkinWeightMax>& DriverSkinWeights, 
	const int32 NumDrivers, const float Sigma)
{	
	const int32 NumDriverTriangles = DriverTriangles.Num();
	const int32 NDriverTriangles = FMath::Clamp(NumDrivers, 1, NumDriverTriangles);

	// Get Distances from Vertex to Each Triangle (ClosestPoint)
	TArray<TPair<float, int32>> SortedDistances;
	TArray<FVector3f> ClosestPoints;
	{
		SortedDistances.SetNumUninitialized(NumDriverTriangles);
		ClosestPoints.SetNumUninitialized(NumDriverTriangles);

		for (int32 DriverTriangleIndex = 0; DriverTriangleIndex < NumDriverTriangles; DriverTriangleIndex++)
		{
			const FIntVector3& DriverTriangle = DriverTriangles[DriverTriangleIndex];
			const FVector3f& A = DriverVertices[DriverTriangle.X];
			const FVector3f& B = DriverVertices[DriverTriangle.Y];
			const FVector3f& C = DriverVertices[DriverTriangle.Z];

			// ClosestPoint To Triangle 
			ClosestPoints[DriverTriangleIndex] = FindClosestPointToTriangle(SourceVertex, A, B, C);

			// Distance To Triangle
			const float Distance = FVector3f::Distance(SourceVertex, ClosestPoints[DriverTriangleIndex]);
			SortedDistances[DriverTriangleIndex] = TPair<float, int32>(Distance, DriverTriangleIndex);
		};

		// Sort By Distance
		SortedDistances.Sort();
	}

	// Get Inverse Distance from Vertex to N-Closest Triangles
	TArray<float> NWeights;
	{
		TArray<FVector3f> NClosestPoints;
		NClosestPoints.SetNumUninitialized(NDriverTriangles);

		for (int32 Index = 0; Index < NDriverTriangles; Index++)
		{
			const int32& DriverTriangleIndex = SortedDistances[Index].Value;
			NClosestPoints[Index] = ClosestPoints[DriverTriangleIndex];
		}

		AnimToTexture_Private::InverseDistanceWeights(SourceVertex, NClosestPoints, NWeights, Sigma);
	}

	DriverTriangleData.Reserve(NDriverTriangles);
	for (int32 Index = 0; Index < NDriverTriangles; Index++)
	{
		const int32& DriverTriangleIndex = SortedDistances[Index].Value;

		if (NWeights[Index] > UE_KINDA_SMALL_NUMBER)
		{
			const FVector3f& ClosestPoint = ClosestPoints[DriverTriangleIndex];
			const FIntVector3& DriverTriangle = DriverTriangles[DriverTriangleIndex];
			const FVector3f& A = DriverVertices[DriverTriangle.X];
			const FVector3f& B = DriverVertices[DriverTriangle.Y];
			const FVector3f& C = DriverVertices[DriverTriangle.Z];

			FSourceVertexDriverTriangleData TriangleData;
			TriangleData.InverseDistanceWeight = NWeights[Index];
			TriangleData.Triangle = DriverTriangle;
			TriangleData.BarycentricCoords = BarycentricCoordinates(ClosestPoint, A, B, C);
			TriangleData.TangentLocalIndex = GetTriangleTangentLocalIndex(ClosestPoint, A, B, C);
			TriangleData.InvMatrix = GetTriangleMatrix(ClosestPoint, A, B, C, TriangleData.TangentLocalIndex).Inverse();

			// Interpolate SkinWeights with Barycentric Coords
			const TArray<VertexSkinWeightMax> SkinWeights = { DriverSkinWeights[TriangleData.Triangle.X], DriverSkinWeights[TriangleData.Triangle.Y], DriverSkinWeights[TriangleData.Triangle.Z] };
			const TArray<float> BarycentricWeights = { TriangleData.BarycentricCoords.X, TriangleData.BarycentricCoords.Y, TriangleData.BarycentricCoords.Z };
			InterpolateVertexSkinWeights(SkinWeights, BarycentricWeights, TriangleData.SkinWeights);

			DriverTriangleData.Add(TriangleData);
		}
	}
}


void FSourceMeshToDriverMesh::Update(const UStaticMesh* StaticMesh, const int32 StaticMeshLODIndex, 
	const USkeletalMesh* SkeletalMesh, const int32 SkeletalMeshLODIndex, 
	const int32 NumDrivers, const float Sigma)
{
	check(StaticMesh);
	check(SkeletalMesh);

	// Get StaticMesh Vertices
	const int32 NumSourceVertices = GetVertices(StaticMesh, StaticMeshLODIndex, SourceVertices, SourceNormals);

	// Get SkeletalMesh Vertices
	const int32 NumDriverVertices = GetVertices(SkeletalMesh, SkeletalMeshLODIndex, DriverVertices);

	// Get SkeletalMesh Triangles
	const int32 NumDriverTriangles = GetTriangles(SkeletalMesh, SkeletalMeshLODIndex, DriverTriangles);

	// Get SkeletalMesh SkinWeights
	GetSkinWeights(SkeletalMesh, SkeletalMeshLODIndex, DriverSkinWeights);

	// Allocate
	SourceVerticesData.SetNumZeroed(NumSourceVertices); // note this is initializing values as zero
	
	// Get SourceVertex -> DriverTriangle Data
	ParallelFor(NumSourceVertices, [&](int32 SourceVertexIndex)
	{	
		// Create Mapping from StaticMesh Vertex to SkeletalMesh Triangles
		SourceVerticesData[SourceVertexIndex].Update(SourceVertices[SourceVertexIndex], 
			DriverVertices, DriverTriangles, DriverSkinWeights, NumDrivers, Sigma);

		// UE_LOGF(LogTemp, Warning, "Vertex: %i NumTriangles: %i.", SourceVertexIndex, SourceVerticesData[SourceVertexIndex].DriverTriangleData.Num());

	});	// end ParallelFor
}

int32 FSourceMeshToDriverMesh::GetNumSourceVertices() const
{
	return SourceVerticesData.Num();
}

int32 FSourceMeshToDriverMesh::GetSourceVertices(TArray<FVector3f>& OutVertices) const
{
	OutVertices = SourceVertices;
	return OutVertices.Num();
}

int32 FSourceMeshToDriverMesh::GetSourceNormals(TArray<FVector3f>& OutNormals) const
{
	OutNormals = SourceNormals;
	return OutNormals.Num();
}

void FSourceMeshToDriverMesh::DeformVerticesAndNormals(const TArray<FVector3f>& InDriverVertices,
	TArray<FVector3f>& OutVertices, TArray<FVector3f>& OutNormals) const
{
	// Source Vertices
	const int32 NumSourceVertices = SourceVerticesData.Num();
	OutVertices.SetNumZeroed(NumSourceVertices);
	OutNormals.SetNumZeroed(NumSourceVertices);

	// Driver Triangles
	const int32 NumDriverTriangles = DriverTriangles.Num();

	// Deform Source Vertices and Normals
	ParallelFor(NumSourceVertices, [&](int32 SourceVertexIndex)
	{
		const FVector3f& SourceVertex = SourceVertices[SourceVertexIndex];
		const FVector3f& SourceNormal = SourceNormals[SourceVertexIndex];

		for (const FSourceVertexDriverTriangleData& TriangleData: SourceVerticesData[SourceVertexIndex].DriverTriangleData)
		{
			const FVector3f& A = InDriverVertices[TriangleData.Triangle.X];
			const FVector3f& B = InDriverVertices[TriangleData.Triangle.Y];
			const FVector3f& C = InDriverVertices[TriangleData.Triangle.Z];

			// Get Driver Triangle Point At Barycentric
			const FVector3f& BarycentricCoords = TriangleData.BarycentricCoords;
			const FVector3f Point = PointAtBarycentricCoordinates(A, B, C, BarycentricCoords);

			// Grt Driver Triangle Matrix
			const uint8& TangentLocalIndex = TriangleData.TangentLocalIndex;
			const FMatrix44f& DriverTriangleInvMatrix = TriangleData.InvMatrix;
			const FMatrix44f DriverTriangleMatrix = GetTriangleMatrix(Point, A, B, C, TangentLocalIndex);

			// Get SourceVertex -> DriverTriangle InverseDistanceWeight
			const float& InverseDistanceWeight = TriangleData.InverseDistanceWeight;

			// Tranform Weighted Source Vertex and Normal
			OutVertices[SourceVertexIndex] += DriverTriangleMatrix.TransformPosition(DriverTriangleInvMatrix.TransformPosition(SourceVertex)) * InverseDistanceWeight;
			OutNormals[SourceVertexIndex] += DriverTriangleMatrix.TransformVector(DriverTriangleInvMatrix.TransformVector(SourceNormal)) * InverseDistanceWeight;
		}

	}); // end ParallelFor
}

void FSourceMeshToDriverMesh::ProjectSkinWeights(TArray<VertexSkinWeightMax>& OutSkinWeights) const
{
	const int32 NumSourceVertices = SourceVerticesData.Num();

	// Allocate
	OutSkinWeights.SetNumUninitialized(NumSourceVertices);

	// Deform Source Vertices and Normals
	ParallelFor(NumSourceVertices, [&](int32 SourceVertexIndex)
	{
		// Allocate
		const int32 Count = SourceVerticesData[SourceVertexIndex].DriverTriangleData.Num();
		
		TArray<VertexSkinWeightMax> SkinWeights;
		SkinWeights.SetNumUninitialized(Count);

		TArray<float> InverseDistanceWeights;
		InverseDistanceWeights.SetNumUninitialized(Count);

		for (int32 Index = 0; Index < Count; Index++)
		{
			SkinWeights[Index] = SourceVerticesData[SourceVertexIndex].DriverTriangleData[Index].SkinWeights;
			InverseDistanceWeights[Index] = SourceVerticesData[SourceVertexIndex].DriverTriangleData[Index].InverseDistanceWeight;
		};

		// Interpolate Driver Weights with InverseDistanceWeighting
		InterpolateVertexSkinWeights(
			SkinWeights,
			InverseDistanceWeights,
			OutSkinWeights[SourceVertexIndex]);

	}); // end ParallelFor
}


} // end namespace AnimToTexture_Private
