// Copyright Epic Games, Inc. All Rights Reserved.
#include "AnimToTextureSkeletalMesh.h"
#include "AnimToTextureEditorModule.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"
#include "Algo/Reverse.h"
#include "Math/NumericLimits.h"
#include "Rendering/SkeletalMeshRenderData.h"
//#include "SkeletalMeshAttributes.h"

namespace AnimToTexture_Private
{

int32 GetVertices(const UStaticMesh* StaticMesh, const int32 LODIndex,
	TArray<FVector3f>& OutPositions, TArray<FVector3f>& OutNormals)
{
	check(StaticMesh);
	OutPositions.Reset();
	OutNormals.Reset();

	if (!StaticMesh->IsMeshDescriptionValid(LODIndex))
	{
		return INDEX_NONE;
	}

	// Get Mesh Description
	const FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);
	check(MeshDescription);

	// Get MeshDescription Vertices
	const FVertexArray& Vertices = MeshDescription->Vertices();
	const int32 NumVertices = Vertices.Num();
	OutPositions.SetNum(NumVertices);
	OutNormals.SetNumZeroed(NumVertices);

	for (const FVertexID& VertexID : Vertices.GetElementIDs())
	{
		const int32 VertexIndex = VertexID.GetValue();

		// Get Vertex Position
		OutPositions[VertexIndex] = MeshDescription->GetVertexPosition(VertexID);

		// Get Avg Vertex Normals
		const TArrayView<const FVertexInstanceID> VertexInstances = MeshDescription->GetVertexVertexInstanceIDs(VertexID);
		for (const FVertexInstanceID VertexInstanceID : VertexInstances)
		{
			const FVector3f VertexInstanceNormal = MeshDescription->VertexInstanceAttributes().GetAttribute<FVector3f>(VertexInstanceID, MeshAttribute::VertexInstance::Normal);
			OutNormals[VertexIndex] += VertexInstanceNormal / VertexInstances.Num();
		}
	}

	return NumVertices;
}

int32 GetVertices(const USkeletalMesh* SkeletalMesh, const int32 LODIndex,
	TArray<FVector3f>& OutPositions)
{
	check(SkeletalMesh);
	OutPositions.Reset();

	const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
	check(RenderData);

	if (!RenderData->LODRenderData.IsValidIndex(LODIndex))
	{
		return INDEX_NONE;
	}

	// Get LOD Data
	const FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LODIndex];

	// Get Total Num of Vertices (for all sections)
	const int32 NumVertices = LODRenderData.GetNumVertices();
	OutPositions.SetNumUninitialized(NumVertices);

	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		OutPositions[VertexIndex] = LODRenderData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
	};

	return NumVertices;
}

int32 GetTriangles(const USkeletalMesh* SkeletalMesh, const int32 LODIndex,
	TArray<FIntVector3>& OutTriangles)
{
	check(SkeletalMesh);
	OutTriangles.Reset();

	const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
	check(RenderData);

	if (!RenderData->LODRenderData.IsValidIndex(LODIndex))
	{
		return INDEX_NONE;
	}

	// Get LOD Data
	const FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LODIndex];

	// Get Indices
	const FMultiSizeIndexContainer& IndexBuffer = LODRenderData.MultiSizeIndexContainer;
	TArray<uint32> IndexArray;
	IndexBuffer.GetIndexBuffer(IndexArray);

	// Allocate Triangles
	const uint32 NumTriangles = IndexArray.Num() / 3;
	OutTriangles.SetNumUninitialized(NumTriangles);

	for (uint32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		OutTriangles[TriangleIndex].X = IndexArray[TriangleIndex * 3];
		OutTriangles[TriangleIndex].Y = IndexArray[TriangleIndex * 3 + 1];
		OutTriangles[TriangleIndex].Z = IndexArray[TriangleIndex * 3 + 2];
	}

	return NumTriangles;
};

void GetSkinWeights(const USkeletalMesh* SkeletalMesh, const int32 LODIndex,
	TArray<VertexSkinWeightMax>& OutSkinWeights)
{
	check(SkeletalMesh);
	OutSkinWeights.Reset();

	// Get Render Data
	const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
	check(RenderData);

	if (!RenderData->LODRenderData.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogAnimToTextureEditor, Warning, "Invalid LODIndex: %i", LODIndex)
		return;
	}

	// Get LOD Data
	const FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LODIndex];

	// Get Weights
	const FSkinWeightVertexBuffer* SkinWeightVertexBuffer = LODRenderData.GetSkinWeightVertexBuffer();
	check(SkinWeightVertexBuffer);

	// Get Weights from Buffer.
	// this is size of number of vertices.
	TArray<FSkinWeightInfo> SkinWeightsInfo;
	SkinWeightVertexBuffer->GetSkinWeights(SkinWeightsInfo);

	// Allocated SkinWeightData
	OutSkinWeights.SetNumUninitialized(SkinWeightsInfo.Num());

	// Loop thru vertices
	for (int32 VertexIndex = 0; VertexIndex < SkinWeightsInfo.Num(); VertexIndex++)
	{
		// Find Section From Global Vertex Index
		// NOTE: BoneMap is stored by Section.
		int32 OutSectionIndex;
		int32 OutSectionVertexIndex;
		LODRenderData.GetSectionFromVertexIndex(VertexIndex, OutSectionIndex, OutSectionVertexIndex);

		// Get Section for Vertex.
		const FSkelMeshRenderSection& RenderSection = LODRenderData.RenderSections[OutSectionIndex];

		// Get Vertex Weights
		const FSkinWeightInfo& SkinWeightInfo = SkinWeightsInfo[VertexIndex];

		// Store Weights
		for (int32 Index = 0; Index < MAX_TOTAL_INFLUENCES; Index++)
		{
			const uint8& BoneWeight = SkinWeightInfo.InfluenceWeights[Index];
			const uint16& BoneIndex = SkinWeightInfo.InfluenceBones[Index];
			const uint16& MeshBoneIndex = RenderSection.BoneMap[BoneIndex];

			OutSkinWeights[VertexIndex].BoneWeights[Index] = BoneWeight;
			OutSkinWeights[VertexIndex].MeshBoneIndices[Index] = MeshBoneIndex;
		}
	}
}

// NOTE: Unfortunatelly SkeletalMeshDescription is only generated for LOD0. 
//       Leaving this here for future reference.
/*
int32 GetVertices(const USkeletalMesh* SkeletalMesh, const int32 LODIndex,
	TArray<FVector3f>& OutPositions)
{
	check(SkeletalMesh);
	OutPositions.Reset();

	if (!SkeletalMesh->IsValidLODIndex(LODIndex))
	{
		return INDEX_NONE;
	}

	// Get Mesh Description
	const FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);
	check(MeshDescription)

	const FVertexArray& Vertices = MeshDescription->Vertices();
	const int32 NumVertices = Vertices.Num();

	OutPositions.SetNum(NumVertices);

	for (const FVertexID& VertexID : Vertices.GetElementIDs())
	{
		const int32 VertexIndex = VertexID.GetValue();
		OutPositions[VertexIndex] = MeshDescription->GetVertexPosition(VertexID);
	}

	return NumVertices;
}

int32 GetTriangles(const USkeletalMesh* SkeletalMesh, const int32 LODIndex,
	TArray<FIntVector3>& OutTriangles)
{
	check(SkeletalMesh);
	OutTriangles.Reset();

	if (!SkeletalMesh->IsValidLODIndex(LODIndex))
	{
		return INDEX_NONE;
	}

	const FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);
	check(MeshDescription)

	const FTriangleArray& Triangles = MeshDescription->Triangles();
	const int32 NumTriangles = Triangles.Num();
	OutTriangles.SetNumUninitialized(NumTriangles);

	for (const FTriangleID& TriangleID : Triangles.GetElementIDs())
	{
		const TArrayView<const FVertexID>& TriangleVertices = MeshDescription->GetTriangleVertices(TriangleID);

		const int32 TriangleIndex = TriangleID.GetValue();
		OutTriangles[TriangleIndex].X = TriangleVertices[0].GetValue();
		OutTriangles[TriangleIndex].Y = TriangleVertices[1].GetValue();
		OutTriangles[TriangleIndex].Z = TriangleVertices[2].GetValue();
	}

	return NumTriangles;
};


void GetSkinWeights(const USkeletalMesh* SkeletalMesh, const int32 LODIndex,
	TArray<VertexSkinWeightMax>& OutSkinWeights)
{
	check(SkeletalMesh);
	OutSkinWeights.Reset();

	if (!SkeletalMesh->IsValidLODIndex(LODIndex))
	{
		return;
	}

	FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);

	// Get Vertices
	const FVertexArray& Vertices = MeshDescription->Vertices();
	const int32 NumVertices = Vertices.Num();

	// Get Vertex Weights
	FSkeletalMeshAttributes SkeletalMeshAttributes(*MeshDescription);
	const FSkinWeightsVertexAttributesRef SkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();
	const FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNames = SkeletalMeshAttributes.GetBoneNames();

	const int32 NumBones = SkeletalMeshAttributes.GetNumBones();

	// Allocated SkinWeightData
	OutSkinWeights.SetNumZeroed(NumVertices);

	for (const FVertexID& VertexID : Vertices.GetElementIDs())
	{
		const int32 VertexIndex = VertexID.GetValue();
		const FVertexBoneWeights BoneWeights = SkinWeights.Get(VertexID);
		check(BoneWeights.Num() < MAX_TOTAL_INFLUENCES)

		for (int32 WeightIndex = 0; WeightIndex < BoneWeights.Num(); WeightIndex++)
		{
			// Note: BoneIndex == RawBoneIndex
			const int32 BoneIndex = BoneWeights[WeightIndex].GetBoneIndex();
			const float BoneWeight = BoneWeights[WeightIndex].GetWeight();

			OutSkinWeights[VertexIndex].BoneWeights[WeightIndex] = (uint8)FMath::RoundToInt(BoneWeight * 255.f);
			OutSkinWeights[VertexIndex].MeshBoneIndices[WeightIndex] = BoneIndex;
		}
	}
}
*/

int32 GetNumBones(const USkeletalMesh* SkeletalMesh)
{
	check(SkeletalMesh);
	
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	return RefSkeleton.GetRawBoneNum();
}

/**
* Copied from FAnimationRuntime::GetComponentSpaceTransform
*/
void GetRefBoneTransforms(const USkeletalMesh* SkeletalMesh, TArray<FTransform>& Transforms)
{
	check(SkeletalMesh);
	Transforms.Reset();

	// Get Reference Skeleton
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const TArray<FTransform>& BoneSpaceTransforms = RefSkeleton.GetRawRefBonePose(); // Get only raw bones (no virtual)
	
	const int32 NumTransforms = BoneSpaceTransforms.Num();
	Transforms.SetNumUninitialized(NumTransforms);

	for (int32 BoneIndex = 0; BoneIndex < NumTransforms; ++BoneIndex)
	{
		// initialize to identity since some of them don't have tracks
		int32 IterBoneIndex = BoneIndex;
		FTransform ComponentSpaceTransform = BoneSpaceTransforms[BoneIndex];

		do
		{
			//const int32 ParentIndex = RefSkeleton.GetParentIndex(IterBoneIndex);
			const int32 ParentIndex = RefSkeleton.GetRawParentIndex(IterBoneIndex); // Get only raw bones (no virtual)
			if (ParentIndex != INDEX_NONE)
			{
				ComponentSpaceTransform *= BoneSpaceTransforms[ParentIndex];
			}

			IterBoneIndex = ParentIndex;
		} while (RefSkeleton.IsValidIndex(IterBoneIndex));

		// 
		Transforms[BoneIndex] = ComponentSpaceTransform;
	}
}

void GetBoneNames(const USkeletalMesh* SkeletalMesh, TArray<FName>& OutNames)
{
	check(SkeletalMesh);
	OutNames.Reset();
	
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	
	const int32 NumRawBones = RefSkeleton.GetRawBoneNum();
	OutNames.SetNumUninitialized(NumRawBones);

	for (int32 BoneIndex = 0; BoneIndex < NumRawBones; ++BoneIndex)
	{
		OutNames[BoneIndex] = RefSkeleton.GetBoneName(BoneIndex);
	}
}

FVector3f PointAtBarycentricCoordinates(const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC, const FVector3f& BarycentricCoords)
{
	return (PointA * BarycentricCoords.X) + (PointB * BarycentricCoords.Y) + (PointC * (1.f - BarycentricCoords.X - BarycentricCoords.Y));
};


void InterpolateVertexSkinWeights(const TArray<VertexSkinWeightMax>& SkinWeights, const TArray<float>& Weights,
	VertexSkinWeightMax& OutVertexSkinWeights)
{
	check(SkinWeights.Num() == Weights.Num())

	// Reset Values
	OutVertexSkinWeights.BoneWeights = TStaticArray<uint8, MAX_TOTAL_INFLUENCES>(InPlace, 0);
	OutVertexSkinWeights.MeshBoneIndices = TStaticArray<uint16, MAX_TOTAL_INFLUENCES>(InPlace, 0);

	// Create Sparse Weighted-SkinWeights
	TMap<uint16, float> WeightedSkinWeights;
	WeightedSkinWeights.Reserve(TNumericLimits<uint16>::Max());

	for (int32 Index = 0; Index < SkinWeights.Num(); Index++)
	{
		for (int32 InfluIndex = 0; InfluIndex < MAX_TOTAL_INFLUENCES; InfluIndex++)
		{
			const uint8& BoneWeight = SkinWeights[Index].BoneWeights[InfluIndex];
			const uint16& MeshBoneIndex = SkinWeights[Index].MeshBoneIndices[InfluIndex];

			const float WeightedVertexSkinWeight = (float)BoneWeight / 255.f * Weights[Index];

			if (WeightedVertexSkinWeight > UE_KINDA_SMALL_NUMBER)
			{
				float* Value = WeightedSkinWeights.Find(MeshBoneIndex);

				// Inititialize
				if (Value == nullptr)
				{
					WeightedSkinWeights.Add(MeshBoneIndex, WeightedVertexSkinWeight);
				}
				// Accumulate Weighted VertexSkinWeight
				else
				{
					*Value += WeightedVertexSkinWeight;
				}
			}
		}
	}

	// Convert Weights to uint8 (and get them ready for Sort)
	TArray<TPair<uint8, uint16>> SortedVertexSkinWeights;
	SortedVertexSkinWeights.Reserve(TNumericLimits<uint16>::Max());
	
	for (const auto& Item : WeightedSkinWeights)
	{
		const uint8 BoneWeight = (uint8)FMath::RoundToInt(Item.Value * 255.f);
		const uint16& MeshBoneIndex = Item.Key;
		SortedVertexSkinWeights.Add(TPair<uint8, uint16>(BoneWeight, MeshBoneIndex));
	}

	// Sort Weights InPlace and reverse
	SortedVertexSkinWeights.Sort();
	Algo::Reverse(SortedVertexSkinWeights);
	for (int32 InfluIndex = 0; InfluIndex < FMath::Min(SortedVertexSkinWeights.Num(), MAX_TOTAL_INFLUENCES); InfluIndex++)
	{
		OutVertexSkinWeights.BoneWeights[InfluIndex] = SortedVertexSkinWeights[InfluIndex].Key;
		OutVertexSkinWeights.MeshBoneIndices[InfluIndex] = SortedVertexSkinWeights[InfluIndex].Value;
	}
}

void ReduceSkinWeights(const TArray<VertexSkinWeightMax>& InSkinWeights, 
	TArray<VertexSkinWeightFour>& OutSkinWeights)
{
	OutSkinWeights.SetNumUninitialized(InSkinWeights.Num());

	for (int32 VertexIndex = 0; VertexIndex < InSkinWeights.Num(); VertexIndex++)
	{
		const VertexSkinWeightMax& InVertexSkinWeight = InSkinWeights[VertexIndex];
		VertexSkinWeightFour& OutVertexSkinWeight = OutSkinWeights[VertexIndex];

		float TotalBoneWeight = 0.f;
		for (int32 Index = 0; Index < 4; Index++)
		{
			TotalBoneWeight += (float)InVertexSkinWeight.BoneWeights[Index] / 255.f;
		}

		// We assume Weights are sorted
		for (int32 Index = 0; Index < 4; Index++)
		{
			OutVertexSkinWeight.BoneWeights[Index] = (uint8)FMath::RoundToInt((float)InVertexSkinWeight.BoneWeights[Index] / TotalBoneWeight);
			OutVertexSkinWeight.MeshBoneIndices[Index] = InVertexSkinWeight.MeshBoneIndices[Index];
		}
	}
}

void GetSkinnedVertices(const USkeletalMeshComponent* SkeletalMeshComponent, const int32 LODIndex,
	TArray<FVector3f>& OutPositions)
{
	check(SkeletalMeshComponent);
	OutPositions.Reset();

	// Get SkeletalMesh
	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	check(SkeletalMesh);

	// Get Matrices
	TArray<FMatrix44f> RefToLocals;
	SkeletalMeshComponent->CacheRefToLocalMatrices(RefToLocals);

	// Get Ref-Pose Vertices
	TArray<FVector3f> Vertices;
	const int32 NumVertices = GetVertices(SkeletalMesh, LODIndex, Vertices);
	OutPositions.SetNumUninitialized(NumVertices);

	// TODO: Add Morph Deltas to Vertices.

	// Get Weights
	TArray<VertexSkinWeightMax> SkinWeights;
	GetSkinWeights(SkeletalMesh, LODIndex, SkinWeights);

	for (int32 VertexIndex = 0; VertexIndex < NumVertices; VertexIndex++)
	{
		const FVector3f& Vertex = Vertices[VertexIndex];
		const VertexSkinWeightMax& Weights = SkinWeights[VertexIndex];

		FVector4f SkinnedVertex(0);
		for (int32 Index = 0; Index < MAX_TOTAL_INFLUENCES; Index++)
		{
			const uint8& BoneWeight = Weights.BoneWeights[Index];
			const uint16& MeshBoneIndex = Weights.MeshBoneIndices[Index];

			// no need to transform if bone doesnt have weights.
			if (BoneWeight > 0) 
			{	
				// get matrix
				const FMatrix44f& RefToLocal = RefToLocals[MeshBoneIndex];

				const float Weight = (float)BoneWeight / 255.f;
				SkinnedVertex += RefToLocal.TransformPosition(Vertex) * Weight;
			}
		}

		OutPositions[VertexIndex] = SkinnedVertex;
	};
};

FVector3f FindClosestPointToTriangle(const FVector3f& P, const FVector3f& A, const FVector3f& B, const FVector3f& C)
{
	const FVector3f AB = B - A;
	const FVector3f AC = C - A;
	const FVector3f AP = P - A;

	const float D1 = FVector3f::DotProduct(AB, AP);
	const float D2 = FVector3f::DotProduct(AC, AP);
	if (D1 <= 0.f && D2 <= 0.f)
	{
		return A;
	}

	const FVector3f BP = P - B;
	const float D3 = FVector3f::DotProduct(AB, BP);
	const float D4 = FVector3f::DotProduct(AC, BP);
	if (D3 >= 0.f && D4 <= D3)
	{
		return B;
	}

	const FVector3f CP = P - C;
	const float D5 = FVector3f::DotProduct(AB, CP);
	const float D6 = FVector3f::DotProduct(AC, CP);
	if (D6 >= 0.f && D5 <= D6)
	{
		return C;
	}

	const float VC = D1 * D4 - D3 * D2;
	if (VC <= 0.f && D1 >= 0.f && D3 <= 0.f)
	{
		const float V = D1 / (D1 - D3);
		return A + V * AB;
	}

	const float VB = D5 * D2 - D1 * D6;
	if (VB <= 0.f && D2 >= 0.f && D6 <= 0.f)
	{
		const float V = D2 / (D2 - D6);
		return A + V * AC;
	}

	const float VA = D3 * D6 - D5 * D4;
	if (VA <= 0.f && (D4 - D3) >= 0.f && (D5 - D6) >= 0.f)
	{
		const float V = (D4 - D3) / ((D4 - D3) + (D5 - D6));
		return B + V * (C - B);
	}

	const float Denom = 1.0f / (VA + VB + VC);
	const float V = VB * Denom;
	const float W = VC * Denom;

	return A + V * AB + W * AC;
}

FVector3f GetTriangleNormal(const FVector3f& A, const FVector3f& B, const FVector3f& C)
{
	const FVector3f V0 = B - A;
	const FVector3f V1 = C - A;
	return FVector3f::CrossProduct(V0, V1); // .GetSafeNormal();
}

uint8 GetTriangleTangentLocalIndex(const FVector3f& P, const FVector3f& A, const FVector3f& B, const FVector3f& C)
{
	const TArray<float> Distances = { (A - P).Length(), (B - P).Length(), (C - P).Length() };

	float MaxDistance = 0.f;
	uint8 TangentLocalIndex = 0;
	for (uint8 Index=0; Index < Distances.Num(); Index++)
	{
		if (Distances[Index] > MaxDistance)
		{
			TangentLocalIndex = Index;
			MaxDistance = Distances[Index];
		}
	}

	return TangentLocalIndex;
}



FMatrix44f GetTriangleMatrix(const FVector3f& P, const FVector3f& A, const FVector3f& B, const FVector3f& C, const uint8 TangentLocalIndex)
{
	// Tangent Vector
	FVector3f V0;
	if (TangentLocalIndex == 0)
	{
		V0 = A - P;
	}
	else if (TangentLocalIndex == 1)
	{
		V0 = B - P;
	}
	else if (TangentLocalIndex == 2)
	{
		V0 = C - P;
	}

	const FVector3f V1 = GetTriangleNormal(A, B, C);
	const FVector3f V2 = FVector3f::CrossProduct(V0, V1);

	//FMatrix44f Matrix = FMatrix44f::Identity; // SetAxes doesn't set [3] elements.
	//Matrix.SetAxes(&V0, &V1, &V2, &P);
	//return Matrix;

	return FMatrix44f(V0, V1, V2, P);
}

FVector3f BarycentricCoordinates(const FVector3f& P, const FVector3f& A, const FVector3f& B, const FVector3f& C)
{
	const FVector3f V0 = B - A;
	const FVector3f V1 = C - A;
	const FVector3f V2 = P - A;
	const float D00 = FVector3f::DotProduct(V0, V0);
	const float D01 = FVector3f::DotProduct(V0, V1);
	const float D11 = FVector3f::DotProduct(V1, V1);
	const float D20 = FVector3f::DotProduct(V2, V0);
	const float D21 = FVector3f::DotProduct(V2, V1);
	const float Denom = 1.0f / (D00 * D11 - D01 * D01);

	const float V = (D11 * D20 - D01 * D21) * Denom;
	const float W = (D00 * D21 - D01 * D20) * Denom;
	const float U = 1.0f - V - W;

	return FVector3f(U, V, W);
}

void InverseDistanceWeights(const FVector3f& Point, const TArray<FVector3f>& Points,
	TArray<float>& OutWeights, const float Sigma)
{
	// Allocate
	const int32 Count = Points.Num();
	OutWeights.SetNumZeroed(Count);

	const float SafeSigma = FMath::Max(UE_KINDA_SMALL_NUMBER, Sigma);
	float SumInverseDistance = 0.f;
	TArray<float> InverseDistances;
	InverseDistances.SetNumUninitialized(Count);

	for (int32 Index = 0; Index < Count; Index++)
	{
		const float Distance = FVector3f::Distance(Point, Points[Index]);

		// No need to interpolate if we are right on top of point.
		if (Distance < UE_KINDA_SMALL_NUMBER)
		{
			OutWeights[Index] = 1.f;
			return;
		}

		InverseDistances[Index] = 1.f / FMath::Pow(Distance, SafeSigma);
		SumInverseDistance += InverseDistances[Index];
	}

	// Normalize Weights
	for (int32 Index = 0; Index < Count; Index++)
	{
		OutWeights[Index] = InverseDistances[Index] / SumInverseDistance;
	}
}

} // end namespace AnimToTexture_Private
