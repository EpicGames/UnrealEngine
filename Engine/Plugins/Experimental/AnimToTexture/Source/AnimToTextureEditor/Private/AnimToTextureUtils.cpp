// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimToTextureUtils.h"

namespace AnimToTexture_Private
{

void DecomposeTransformation(const FTransform& Transform, 
	FVector3f& OutTranslation, FVector4f& OutRotation)
{
	// Get Translation
	OutTranslation = (FVector3f)Transform.GetTranslation();

	// Get Rotation 
	const FQuat4f Quat = (FQuat4f)Transform.GetRotation();

	FVector3f Axis;
	float Angle;
	Quat.ToAxisAndAngle(Axis, Angle);

	OutRotation = FVector4f(Axis, Angle);
}

void DecomposeTransformations(const TArray<FTransform>& Transforms, 
	TArray<FVector3f>& OutTranslations, TArray<FVector4f>& OutRotations)
{
	const int32 NumTransforms = Transforms.Num();
	OutTranslations.SetNumUninitialized(NumTransforms);
	OutRotations.SetNumUninitialized(NumTransforms);

	for (int32 Index = 0; Index < NumTransforms; ++Index)
	{
		DecomposeTransformation(Transforms[Index], OutTranslations[Index], OutRotations[Index]);
	}
};

} // end namespace AnimToTexture_Private