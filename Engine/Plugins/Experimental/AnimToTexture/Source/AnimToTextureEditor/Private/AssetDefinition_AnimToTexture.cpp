// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AnimToTexture.h"

#include "AnimToTextureBPLibrary.h"
#include "AnimToTextureDataAsset.h"
#include "AssetViewUtils.h"
#include "ContentBrowserMenuContexts.h"
#include "Materials/MaterialInstanceConstant.h"
#include "ObjectEditorUtils.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenus.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "UAssetDefinition_AnimToTexture"

FText UAssetDefinition_AnimToTexture::GetAssetDisplayName() const
{
	return LOCTEXT("AnimToTextureAssetActions", "AnimToTexture");
}

FLinearColor UAssetDefinition_AnimToTexture::GetAssetColor() const
{
	return FColor::Blue;
}

TSoftClassPtr<UObject> UAssetDefinition_AnimToTexture::GetAssetClass() const
{
	return UAnimToTextureDataAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AnimToTexture::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { EAssetCategoryPaths::Animation };
	return Categories;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_AnimToTexture
{
	void RunAnimToTexture(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UAnimToTextureDataAsset*> SelectedAnimToTextureObjects = Context->LoadSelectedObjects<UAnimToTextureDataAsset>();

		for (auto ObjIt = SelectedAnimToTextureObjects.CreateConstIterator(); ObjIt; ++ObjIt)
		{
			UAnimToTextureDataAsset* DataAsset = *ObjIt;
			// Create UVs and Textures
			if (UAnimToTextureBPLibrary::AnimationToTexture(*ObjIt))
			{
				// Update Material Instances (if Possible)
				if (UStaticMesh* StaticMesh = DataAsset->GetStaticMesh())
				{
					for (FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
					{
						if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(StaticMaterial.MaterialInterface))
						{
							UAnimToTextureBPLibrary::UpdateMaterialInstanceFromDataAsset(DataAsset, MaterialInstanceConstant, EMaterialParameterAssociation::GlobalParameter);
						}
					}
				}
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UAnimToTextureDataAsset::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					const TAttribute<FText> Label = LOCTEXT("AnimToTexture_Run", "Run Animation To Texture");
					const TAttribute<FText> ToolTip = LOCTEXT("AnimToTexture_RunTooltip", "Creates Vertex Animation Textures (VAT)");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&RunAnimToTexture);

					InSection.AddMenuEntry(TEXT("AnimToTexture_RunAnimationToTexture"), Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
};

//--------------------------------------------------------------------
// Menu Extensions

#undef LOCTEXT_NAMESPACE
