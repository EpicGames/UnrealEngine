// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimToTextureDataAsset.h"
#include "AnimToTextureMeshMapping.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Materials/MaterialLayersFunctions.h"

#include "AnimToTextureBPLibrary.generated.h"

class UStaticMesh;
class USkeletalMesh;

/**
* Converts a series of animation frames into a Vertex Animation Texture (VAT)
* 
* The texture can store Vertex positions and normals or Bone positions and rotations.
*/
UCLASS()
class UAnimToTextureBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR

	/**
	* Bakes Animation Data into Textures.
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "AnimToTexture"))
	static ANIMTOTEXTUREEDITOR_API UPARAM(DisplayName="bSuccess") bool AnimationToTexture(UAnimToTextureDataAsset* DataAsset);

	/** 
	* Utility for converting SkeletalMesh into a StaticMesh
	*/
	UFUNCTION(BlueprintCallable, Category = "AnimToTexture")
	static ANIMTOTEXTUREEDITOR_API UStaticMesh* ConvertSkeletalMeshToStaticMesh(USkeletalMesh* SkeletalMesh, const FString PackageName, const int32 LODIndex = -1);

	/**
	* Utility for setting a StaticMesh LightMapIndex.
	*/
	UFUNCTION(BlueprintCallable, Category = "AnimToTexture")
	static ANIMTOTEXTUREEDITOR_API UPARAM(DisplayName = "bSuccess") bool SetLightMapIndex(UStaticMesh* StaticMesh, const int32 LODIndex, const int32 LightmapIndex=1, bool bGenerateLightmapUVs=true);

	/**
	* Updates a material's parameters to match those of an AnimToTexture DataAsset
	*/
	UFUNCTION(BlueprintCallable, meta = (Category = "AnimToTexture"))
	static ANIMTOTEXTUREEDITOR_API void UpdateMaterialInstanceFromDataAsset(const UAnimToTextureDataAsset* DataAsset, class UMaterialInstanceConstant* MaterialInstance,
		const EMaterialParameterAssociation MaterialParameterAssociation = EMaterialParameterAssociation::GlobalParameter);

#endif // WITH_EDITOR

private:

	// Runs some validations for the assets in DataAsset
	// Returns false if there is any problems with the data, warnings will be printed in Log
	static bool CheckDataAsset(const UAnimToTextureDataAsset* DataAsset,
		int32& OutSocketIndex, TArray<FAnimToTextureAnimSequenceInfo>& OutAnimSequences);

	// Returns Start, EndFrame and NumFrames in Animation
	static int32 GetAnimationFrameRange(const FAnimToTextureAnimSequenceInfo& Animation, 
		int32& OutStartFrame, int32& OutEndFrame);

	// Get Vertex and Normals from Current Pose
	// The VertexDelta is returned from the RefPose
	static void GetVertexDeltasAndNormals(const USkeletalMeshComponent* SkeletalMeshComponent, const int32 LODIndex, 
		const AnimToTexture_Private::FSourceMeshToDriverMesh& SourceMeshToDriverMesh,
		const FTransform RootTransform,
		TArray<FVector3f>& OutVertexDeltas, TArray<FVector3f>& OutVertexNormals);

	// Gets RefPose Bone Position and Rotations.
	static int32 GetRefBonePositionsAndRotations(const USkeletalMesh* SkeletalMesh, 
		TArray<FVector3f>& OutBoneRefPositions, TArray<FVector4f>& OutBoneRefRotations);

	// Gets Bone Position and Rotations for Current Pose.	
	// The BonePosition is returned relative to the RefPose
	static int32 GetBonePositionsAndRotations(const USkeletalMeshComponent* SkeletalMeshComponent, const TArray<FVector3f>& BoneRefPositions,
		TArray<FVector3f>& BonePositions, TArray<FVector4f>& BoneRotations);

	// Normalizes Deltas and Normals between [0-1] with Bounding Box
	static void NormalizeVertexData(
		const TArray<FVector3f>& Deltas, const TArray<FVector3f>& Normals,
		FVector3f& OutMinBBox, FVector3f& OutSizeBBox,
		TArray<FVector3f>& OutNormalizedDeltas, TArray<FVector3f>& OutNormalizedNormals);

	// Normalizes Positions and Rotations between [0-1] with Bounding Box
	static void NormalizeBoneData(
		const TArray<FVector3f>& Positions, const TArray<FVector4f>& Rotations,
		FVector3f& OutMinBBox, FVector3f& OutSizeBBox,
		TArray<FVector3f>& OutNormalizedPositions, TArray<FVector4f>& OutNormalizedRotations);

	/* Returns best resolution for the given data. 
	*  Returns false if data doesnt fit in the the max range */
	static bool FindBestResolution(const int32 NumFrames, const int32 NumElements,
								   int32& OutHeight, int32& OutWidth, int32& OutRowsPerFrame,
								   const int32 MaxHeight = 4096, const int32 MaxWidth = 4096, bool bEnforcePowerOfTwo = false);

	/* Sets Static Mesh FullPrecisionUVs Property*/
	static void SetFullPrecisionUVs(UStaticMesh* StaticMesh, const int32 LODIndex, bool bFullPrecision=true);

	/* Sets Static Mesh Bound Extensions */
	static void SetBoundsExtensions(UStaticMesh* StaticMesh, const FVector& MinBBox, const FVector& SizeBBox);

	/* Creates UV Coord with vertices */
	static bool CreateUVChannel(UStaticMesh* StaticMesh, const int32 LODIndex, const int32 UVChannelIndex,
		const int32 Height, const int32 Width);

};
