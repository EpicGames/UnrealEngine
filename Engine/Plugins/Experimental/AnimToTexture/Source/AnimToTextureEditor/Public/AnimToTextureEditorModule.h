// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAnimToTextureEditor, Log, All);

class FAnimToTextureEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
