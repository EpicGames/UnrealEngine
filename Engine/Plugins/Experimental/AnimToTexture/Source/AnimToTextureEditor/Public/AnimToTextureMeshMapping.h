// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimToTextureSkeletalMesh.h"
#include "CoreMinimal.h"

namespace AnimToTexture_Private
{

struct FSourceVertexDriverTriangleData
{
	uint8               TangentLocalIndex;
	float               InverseDistanceWeight;
	FIntVector3         Triangle;
	FVector3f           BarycentricCoords;
	FMatrix44f          InvMatrix;
	VertexSkinWeightMax SkinWeights;
};

class FSourceVertexData
{
public:

	FSourceVertexData() = default;	
	
	void Update(const FVector3f& SourceVertex,
		const TArray<FVector3f>& DriverVertices, const TArray<FIntVector3>& DriverTriangles, const TArray<VertexSkinWeightMax>& DriverSkinWeights, 
		const int32 NumDrivers, const float Sigma=1.f);

	// DriverTriangle Data specific to this SourceVertex
	TArray<FSourceVertexDriverTriangleData> DriverTriangleData;
	
};

// Creates Mapping between StaticMesh (Source) and SkeletalMesh (Driver)
class FSourceMeshToDriverMesh
{
public:

	FSourceMeshToDriverMesh() = default;
	
	void Update(const UStaticMesh* StaticMesh, const int32 StaticMeshLODIndex,
		const USkeletalMesh* SkeletalMesh, const int32 SkeletalMeshLODIndex, 
		const int32 NumDrivers, const float Sigma=1.f);

	// Returns Number of Source Vertices
	int32 GetNumSourceVertices() const;
	// Returns Source Vertices
	int32 GetSourceVertices(TArray<FVector3f>& OutVertices) const;
	// Returns Source Normals
	int32 GetSourceNormals(TArray<FVector3f>& OutNormals) const;
	
	// Deforms Source Vertices with Driver Triangles
	void DeformVerticesAndNormals(const TArray<FVector3f>& DriverVertices, 
		TArray<FVector3f>& OutVertices, TArray<FVector3f>& OutNormals) const;

	// Project SkinWeights
	void ProjectSkinWeights(TArray<VertexSkinWeightMax>& OutSkinWeights) const;

private:

	// Size of Source Mesh
	TArray<FVector3f>         SourceVertices;
	TArray<FVector3f>         SourceNormals;
	TArray<FSourceVertexData> SourceVerticesData;

	// Driver Data
	TArray<FVector3f>   DriverVertices;
	TArray<FIntVector3> DriverTriangles;
	TArray<VertexSkinWeightMax> DriverSkinWeights;

};

} // end namespace AnimToTexture_Private
