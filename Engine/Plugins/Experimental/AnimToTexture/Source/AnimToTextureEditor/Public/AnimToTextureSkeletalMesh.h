// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimToTextureDataAsset.h"
#include "CoreMinimal.h"
#include "Containers/StaticArray.h"
#include "GPUSkinPublicDefs.h" 

namespace AnimToTexture_Private
{

template <uint16 NumInfluences>
struct TVertexSkinWeight
{
	TStaticArray<uint16, NumInfluences> MeshBoneIndices;
	TStaticArray<uint8, NumInfluences>  BoneWeights;
};

using VertexSkinWeightMax  = TVertexSkinWeight<MAX_TOTAL_INFLUENCES>;
using VertexSkinWeightFour = TVertexSkinWeight<4>;

/* Returns StaticMesh Vertex Positions */
int32 GetVertices(const UStaticMesh* StaticMesh, const int32 LODIndex,
	TArray<FVector3f>& OutPositions, TArray<FVector3f>& OutNormals);

/* Returns RefPose Vertex Positions */
int32 GetVertices(const USkeletalMesh* SkeletalMesh, const int32 LODIndex,
	TArray<FVector3f>& OutPositions);

/* Returns Triangles vertex indices */
int32 GetTriangles(const USkeletalMesh* SkeletalMesh, const int32 LODIndex,
	TArray<FIntVector3>& OutTriangles);

/* Computes CPUSkinning at Pose */
void GetSkinnedVertices(const USkeletalMeshComponent* SkeletalMeshComponent, const int32 LODIndex,
	TArray<FVector3f>& OutPositions);

/** Gets Skin Weights Data from SkeletalMeshComponent */
void GetSkinWeights(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, 
	TArray<VertexSkinWeightMax>& OutSkinWeights);

/** Reduce Weights from MAX_TOTAL_INFLUENCES to 4 */
void ReduceSkinWeights(const TArray<VertexSkinWeightMax>& InSkinWeights, TArray<VertexSkinWeightFour>& OutSkinWeights);

/* Interpolates an Array of SkinWeights with an Array of Weights(InverseDistanceWeights) */
void InterpolateVertexSkinWeights(const TArray<VertexSkinWeightMax>& VertexSkinWeights, const TArray<float>& Weights,
	VertexSkinWeightMax& OutVertexSkinWeights);

/* Returns Number of RawBones (no virtual bones)*/
int32 GetNumBones(const USkeletalMesh* SkeletalMesh);

/* Returns RefPose Bone Transforms.
   Only the RawBones are returned (no virtual bones)
   The returned transforms are in ComponentSpace */
void GetRefBoneTransforms(const USkeletalMesh* SkeletalMesh, TArray<FTransform>& OutTransforms);

/* Returns Bone Names. 
   Only the RawBones are returned (no virtual bones) */
void GetBoneNames(const USkeletalMesh* SkeletalMesh, TArray<FName>& OutNames);

/* Computes closest point to triangle */
FVector3f FindClosestPointToTriangle(const FVector3f& Point, const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC);

/* Computes Barycentric coordinates from point to triangle */
FVector3f BarycentricCoordinates(const FVector3f& Point, const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC);

/* Returns Point at Barycentric coordinates of the given triangle */
FVector3f PointAtBarycentricCoordinates(const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC, const FVector3f& BarycentricCoords);

/* Computes Triangle Normal (not normalized) */
FVector3f GetTriangleNormal(const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC);

/* Returns TangentLocalIndex for Triangle 
* TangentLocalIndex 0: PointA - PointP
* TangentLocalIndex 1: PointB - PointP
* TangentLocalIndex 2: PointC - PointP */
uint8 GetTriangleTangentLocalIndex(const FVector3f& Point, const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC);

/* Computes Triangle Matrix */
FMatrix44f GetTriangleMatrix(const FVector3f& Point, const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC, const uint8 TangentLocalIndex);

// Returns a list of weights from Point to each of the given Points.
// The returned Weights are the size of Points and normalized (sum to 1.0) 
void InverseDistanceWeights(const FVector3f& Point, const TArray<FVector3f>& Points,
	TArray<float>& OutWeights, const float Sigma = 1.f);

} // end namespace AnimToTexture_Private
