// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimToTextureSkeletalMesh.h"
#include "CoreMinimal.h"
#include "TextureResource.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"

namespace AnimToTexture_Private
{

struct FVector4u16
{
	uint16 X;
	uint16 Y;
	uint16 Z;
	uint16 W;
};

struct FLowPrecision
{
	using ColorType = FColor;
	static constexpr EPixelFormat PixelFormat = EPixelFormat::PF_B8G8R8A8;
	static constexpr ETextureSourceFormat TextureSourceFormat = ETextureSourceFormat::TSF_BGRA8;
	static constexpr TextureCompressionSettings CompressionSettings = TextureCompressionSettings::TC_VectorDisplacementmap;
	static constexpr ColorType DefaultColor = FColor(0, 0, 0, 0);		
};

struct FHighPrecision
{
	using ColorType = FVector4u16;
	static constexpr EPixelFormat PixelFormat = EPixelFormat::PF_R16G16B16A16_UNORM;
	static constexpr ETextureSourceFormat TextureSourceFormat = ETextureSourceFormat::TSF_RGBA16;
	static constexpr TextureCompressionSettings CompressionSettings = TextureCompressionSettings::TC_HDR;
	static constexpr ColorType DefaultColor = { 0, 0, 0, 0 };
};

/** Writes list of vectors into texture
*   Note: They must be pre-normalized. */
template<class V, class TextureSettings>
bool WriteVectorsToTexture(const TArray<V>& Vectors,
	const int32 NumFrames, const int32 RowsPerFrame,
	const int32 Height, const int32 Width, 
	UTexture2D* Texture);

/* Writes list of skinweights into texture.
*  The SkinWeights data is already in uint8 & uint16 format, no need for normalizing it.
*/
template<class TextureSettings>
bool WriteSkinWeightsToTexture(const TArray<VertexSkinWeightFour>& SkinWeights, const int32 NumBones,
	const int32 RowsPerFrame,
	const int32 Height, const int32 Width,
	UTexture2D* Texture);

/* Helper utility for writing 8 or 16 bits textures */
template<class TextureSettings>
bool WriteToTexture(UTexture2D* Texture, const uint32 Height, const uint32 Width, const TArray<typename TextureSettings::ColorType>& Data);

template<class V /* FVector3f / FVector4f */, class C /* FColor / FVector4u16 */>
void VectorToColor(const V& Vector, C& Color);

/* Decomposes Transform in Translation and AxisAndAngle */
void DecomposeTransformation(const FTransform& Transform, FVector3f& OutTranslation, FVector4f& OutRotation);
void DecomposeTransformations(const TArray<FTransform>& Transforms, TArray<FVector3f>& OutTranslations, TArray<FVector4f>& OutRotations);

} // end namespace AnimToTexture_Private

// ----------------------------------------------------------------------------

// LowPrecision
template<>
inline void AnimToTexture_Private::VectorToColor(const FVector3f& Vector, FColor& Color)
{
	const float ClampedX = FMath::Clamp(Vector.X, 0.f, 1.f);
	const float ClampedY = FMath::Clamp(Vector.Y, 0.f, 1.f);
	const float ClampedZ = FMath::Clamp(Vector.Z, 0.f, 1.f);

	Color.R = (uint8)FMath::RoundToInt(ClampedX * 255.f);
	Color.G = (uint8)FMath::RoundToInt(ClampedY * 255.f);
	Color.B = (uint8)FMath::RoundToInt(ClampedZ * 255.f);
	Color.A = TNumericLimits<uint8>::Max();
}

// LowPrecision
template<>
inline void AnimToTexture_Private::VectorToColor(const FVector4f& Vector, FColor& Color)
{
	const float ClampedX = FMath::Clamp(Vector.X, 0.f, 1.f);
	const float ClampedY = FMath::Clamp(Vector.Y, 0.f, 1.f);
	const float ClampedZ = FMath::Clamp(Vector.Z, 0.f, 1.f);
	const float ClampedW = FMath::Clamp(Vector.W, 0.f, 1.f);
	
	Color.R = (uint8)FMath::RoundToInt(ClampedX * 255.f);
	Color.G = (uint8)FMath::RoundToInt(ClampedY * 255.f);
	Color.B = (uint8)FMath::RoundToInt(ClampedZ * 255.f);
	Color.A = (uint8)FMath::RoundToInt(ClampedW * 255.f);
}

// HighPrecision
template<>
inline void AnimToTexture_Private::VectorToColor(const FVector3f& Vector, FVector4u16& Color)
{
	Color.X = FMath::RoundToInt(FMath::Clamp(Vector.X, 0.f, 1.f) * TNumericLimits<uint16>::Max());
	Color.Y = FMath::RoundToInt(FMath::Clamp(Vector.Y, 0.f, 1.f) * TNumericLimits<uint16>::Max());
	Color.Z = FMath::RoundToInt(FMath::Clamp(Vector.Z, 0.f, 1.f) * TNumericLimits<uint16>::Max());
	Color.W = TNumericLimits<uint16>::Max();
}

// HighPrecision
template<>
inline void AnimToTexture_Private::VectorToColor(const FVector4f& Vector, FVector4u16& Color)
{
	Color.X = FMath::RoundToInt(FMath::Clamp(Vector.X, 0.f, 1.f) * TNumericLimits<uint16>::Max());
	Color.Y = FMath::RoundToInt(FMath::Clamp(Vector.Y, 0.f, 1.f) * TNumericLimits<uint16>::Max());
	Color.Z = FMath::RoundToInt(FMath::Clamp(Vector.Z, 0.f, 1.f) * TNumericLimits<uint16>::Max());
	Color.W = FMath::RoundToInt(FMath::Clamp(Vector.W, 0.f, 1.f) * TNumericLimits<uint16>::Max());
}

template<class V, class TextureSettings>
inline bool AnimToTexture_Private::WriteVectorsToTexture(const TArray<V>& Vectors,
	const int32 NumFrames, const int32 RowsPerFrame,
	const int32 Height, const int32 Width, UTexture2D* Texture)
{
	if (!Texture || !NumFrames)
	{
		return false;
	}

	// NumElements Per-Frame
	const int32 NumElements = Vectors.Num() / NumFrames;

	// Allocate PixelData.
	TArray<typename TextureSettings::ColorType> Pixels;
	Pixels.Init(TextureSettings::DefaultColor, Height * Width);

	// Fillout Frame Data
	for (int32 Frame = 0; Frame < NumFrames; Frame++)
	{
		const int32 BlockStart = RowsPerFrame * Width * Frame;

		// Set Data.
		for (int32 Index = 0; Index < NumElements; Index++)
		{
			const V& Vector = Vectors[NumElements * Frame + Index];
			typename TextureSettings::ColorType& Pixel = Pixels[BlockStart + Index];

			VectorToColor<V, typename TextureSettings::ColorType>(Vector, Pixel);
		}
	}

	// Write to Texture
	return WriteToTexture<TextureSettings>(Texture, Height, Width, Pixels);
}

template<class TextureSettings>
inline bool AnimToTexture_Private::WriteSkinWeightsToTexture(const TArray<AnimToTexture_Private::VertexSkinWeightFour>& SkinWeights, const int32 NumBones,
	const int32 RowsPerFrame, const int32 Height, const int32 Width, UTexture2D* Texture)
{
	check(Texture);
	
	const int32 NumVertices = SkinWeights.Num();

	// Allocate PixelData.
	TArray<typename TextureSettings::ColorType> Pixels;
	Pixels.Init(TextureSettings::DefaultColor, Height * Width);

	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		const VertexSkinWeightFour& VertexSkinWeight = SkinWeights[VertexIndex];

		// Normalize BoneIndices
		const FVector4f BoneIndices(
			(float)VertexSkinWeight.MeshBoneIndices[0] / float(NumBones),
			(float)VertexSkinWeight.MeshBoneIndices[1] / float(NumBones),
			(float)VertexSkinWeight.MeshBoneIndices[2] / float(NumBones),
			(float)VertexSkinWeight.MeshBoneIndices[3] / float(NumBones));

		// Normalize BoneWeights
		const FVector4f BoneWeights(
			(float)VertexSkinWeight.BoneWeights[0] / 255.f,
			(float)VertexSkinWeight.BoneWeights[1] / 255.f,
			(float)VertexSkinWeight.BoneWeights[2] / 255.f,
			(float)VertexSkinWeight.BoneWeights[3] / 255.f);

		// Write BoneIndex
		{
			typename TextureSettings::ColorType& Pixel = Pixels[VertexIndex];
			VectorToColor<FVector4f, typename TextureSettings::ColorType>(BoneIndices, Pixel);
		}
		
		// Write BoneWeight
		{
			typename TextureSettings::ColorType& Pixel = Pixels[RowsPerFrame * Width + VertexIndex];
			VectorToColor<FVector4f, typename TextureSettings::ColorType>(BoneWeights, Pixel);
		}
	};

	// Write to Texture
	return WriteToTexture<TextureSettings>(Texture, Height, Width, Pixels);
}

template<class TextureSettings>
inline bool AnimToTexture_Private::WriteToTexture(
	UTexture2D* Texture,
	const uint32 Height, const uint32 Width,
	const TArray<typename TextureSettings::ColorType>& Pixels)
{
	check(Texture);

	// TODO FIX ME: this is the wrong way to build a UTexture
	// instead fill out an FImage
	// and use Texture.Source.Init() from FImage
	// there is no reason to be touching the PlatformData or BulkData
	
	Texture->PreEditChange(nullptr);

	// ------------------------------------------------------------------------
	// Get Texture Platform
	FTexturePlatformData* PlatformData = Texture->GetPlatformData();
	if (!PlatformData)
	{	
		PlatformData = new FTexturePlatformData();
		Texture->SetPlatformData(PlatformData);
	}
	PlatformData->SizeX = Width;
	PlatformData->SizeY = Height;
	PlatformData->SetNumSlices(1);
	PlatformData->PixelFormat = TextureSettings::PixelFormat;
	
	// ------------------------------------------------------------------------
	// Get First MipMap
	//
	FTexture2DMipMap* Mip;
	if (PlatformData->Mips.IsEmpty())
	{
		Mip = new FTexture2DMipMap(0, 0);
		PlatformData->Mips.Add(Mip);
	}
	else
	{
		Mip = &PlatformData->Mips[0];

	}
	Mip->SizeX = Width;
	Mip->SizeY = Height;
	Mip->SizeZ = 1;
	
	// ------------------------------------------------------------------------
	// Lock the Mipmap data so it can be modified
	Mip->BulkData.Lock(LOCK_READ_WRITE);

	// Reallocate MipMap
	uint8* TextureData = (uint8*)Mip->BulkData.Realloc(Width * Height * sizeof(typename TextureSettings::ColorType));

	// Copy the pixel data into the Texture data	
	const uint8* PixelsData = (uint8*)Pixels.GetData();
	FMemory::Memcpy(TextureData, PixelsData, Width * Height * sizeof(typename TextureSettings::ColorType));

	// Unlock data
	Mip->BulkData.Unlock();

	// Initialize a new texture
	Texture->Source.Init(Width, Height, 1, 1, TextureSettings::TextureSourceFormat, PixelsData);
	
	// Set parameters
	Texture->SRGB = 0;
	Texture->Filter = TextureFilter::TF_Nearest;
	Texture->CompressionSettings = TextureSettings::CompressionSettings;
	Texture->MipGenSettings = TextureMipGenSettings::TMGS_NoMipmaps;

	// Update and Mark to Save.
	Texture->PostEditChange();

	return true;
}
